[CoreRedirects]

+StructRedirects=(OldName="BindableStructDesc",NewName="/script/PropertyBindingUtils.PropertyBindingBindableStructDescriptor")
+EnumRedirects=(OldName="EPropertyBindingAccessType",NewName="/Script/PropertyBindingUtils.EPropertyBindingPropertyAccessType")
