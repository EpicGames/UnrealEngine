{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "PropertyBindingUtils",
	"Description" : "Utility code for implementing property bindings",
	"Category" : "Gameplay",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules":
	[
		{
			"Name": "PropertyBindingUtils",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "PropertyBindingUtilsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "PropertyBindingUtilsTestSuite",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins":
	[
	]
}