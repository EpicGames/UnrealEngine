// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBindingBinding.h"
#include "PropertyBindingDataView.h"
#include "PropertyBindingBindableStructDescriptor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyBindingBinding)

FString UE::PropertyBinding::GetDescriptorAndPathAsString(const FPropertyBindingBindableStructDescriptor& InDescriptor
	, const FPropertyBindingPath& InPath)
{
	TStringBuilder<256> Result;

	Result << InDescriptor.ToString();

	if (!InPath.IsPathEmpty())
	{
		Result << TEXT(" ");
		Result << InPath.ToString();
	}

	return FString(Result);
}

FString FPropertyBindingBinding::ToString() const
{
	TStringBuilder<256> Result;
	Result << SourcePropertyPath.ToString();
	Result << TEXT(" --> ");
	Result << TargetPropertyPath.ToString();
	return FString(Result);
}
