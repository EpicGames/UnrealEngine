// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBindingBindingCollection.h"
#include "PropertyBindingBinding.h"
#include "PropertyBindingBindingCollectionOwner.h"
#include "PropertyBindingDataView.h"
#include "StructUtils/InstancedStructContainer.h"
#include "VisualLogger/VisualLogger.h"

#if WITH_EDITOR
#include "PropertyBindingPath.h"
#endif // WITH_EDITOR

#include "Algo/Copy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyBindingBindingCollection)

#if WITH_EDITOR
#define PROPERTY_BINDINGS_LOG(LogOwner, Verbosity, Format, ...) \
	if (LogOwner) \
	{ \
		UE_VLOG_UELOG(LogOwner, LogPropertyBindingUtils, Verbosity, Format, ##__VA_ARGS__); \
	} \
	else \
	{ \
		UE_LOG(LogPropertyBindingUtils, Verbosity, Format, ##__VA_ARGS__); \
	}
#else
#define PROPERTY_BINDINGS_LOG(LogOwner, Verbosity, Format, ...) UE_LOG(LogPropertyBindingUtils, Verbosity, Format, ##__VA_ARGS__);
#endif // WITH_EDITOR

#define PROPERTY_BINDINGS_CLOG(Condition, LogOwner, Verbosity, Format, ...) \
	if (Condition) { PROPERTY_BINDINGS_LOG(LogOwner, Verbosity, Format, ##__VA_ARGS__); }

const UObject* FPropertyBindingBindingCollection::GetLogOwner() const
{
#if WITH_EDITOR
	return Cast<UObject>(GetBindingsOwner());
#else
	return nullptr;
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void FPropertyBindingBindingCollection::AddBinding(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath)
{
	RemoveBindings(InTargetPath, ESearchMode::Exact);

	if (FPropertyBindingBinding* AddedBinding = AddBindingInternal(InSourcePath, InTargetPath))
	{
		UpdateSegmentsForNewlyAddedBinding(*AddedBinding);
	}
}

void FPropertyBindingBindingCollection::RemoveBindings(const FPropertyBindingPath& InTargetPath, ESearchMode InSearchMode)
{
	if (InSearchMode == ESearchMode::Exact)
	{
		RemoveBindingsInternal([LogOwner = GetLogOwner(), &InTargetPath](FPropertyBindingBinding& Binding)
			{
				if (Binding.GetTargetPath() == InTargetPath)
				{
					PROPERTY_BINDINGS_LOG(LogOwner, Log, TEXT("Removing binding using same target path: '%s' == '%s'")
						, *Binding.GetTargetPath().ToString(), *InTargetPath.ToString());
					return true;
				}
				return false;
			});
	}
	else
	{
		RemoveBindingsInternal([LogOwner = GetLogOwner(), &InTargetPath](FPropertyBindingBinding& Binding)
			{
				if (Binding.GetTargetPath().Includes(InTargetPath))
				{
					PROPERTY_BINDINGS_LOG(LogOwner, Log, TEXT("Removing binding using target sub path: '%s' contains '%s'")
						, *Binding.GetTargetPath().ToString(), *InTargetPath.ToString());
					return true;
				}
				return false;
			});
	}
}

void FPropertyBindingBindingCollection::RemoveBindings(TFunctionRef<bool(FPropertyBindingBinding&)> InPredicate)
{
	RemoveBindingsInternal(InPredicate);
}

void FPropertyBindingBindingCollection::CopyBindings(const FGuid InFromStructID, const FGuid InToStructID)
{
	CopyBindingsInternal(InFromStructID, InToStructID);
}

void FPropertyBindingBindingCollection::CopyBindingsInternal(const FGuid InFromStructID, const FGuid InToStructID)
{
	CopyBindingsImplementation(InFromStructID, InToStructID, [](const FPropertyBindingBinding&){ return true; });
}

void FPropertyBindingBindingCollection::CopyBindingsImplementation(const FGuid InFromStructID, const FGuid InToStructID, TFunctionRef<bool(const FPropertyBindingBinding& Binding)> CanCopy)
{
	// Find the StructID and copy the binding. If we find functions, then copy the function and copy the functions binding recursively.
	TArray<TTuple<FPropertyBindingPath, FPropertyBindingPath>, TInlineAllocator<8>> BindingsToCopy;
	ForEachBinding([&InFromStructID, &InToStructID, &BindingsToCopy, &CanCopy](const FPropertyBindingBinding& Binding)
		{
			if (Binding.GetTargetPath().GetStructID() == InFromStructID)
			{
				if (CanCopy(Binding))
				{
					BindingsToCopy.Emplace(Binding.GetSourcePath(), FPropertyBindingPath(InToStructID, Binding.GetTargetPath().GetSegments()));
				}
			}
		});

	// Copy all bindings that target "FromStructID" and retarget them to "ToStructID".
	for (const TTuple<FPropertyBindingPath, FPropertyBindingPath>& BindingToCopy : BindingsToCopy)
	{
		AddBindingInternal(BindingToCopy.Get<0>(), BindingToCopy.Get<1>());
	}
}

void FPropertyBindingBindingCollection::UpdateSegmentsForNewlyAddedBinding(FPropertyBindingBinding& AddedBinding)
{
	// If we have bindings owner, update property path segments to capture property IDs, etc.
	const IPropertyBindingBindingCollectionOwner* PropertyBindingsOwner = BindingsOwner.GetInterface();
	if (PropertyBindingsOwner != nullptr)
	{
		FPropertyBindingBinding& Binding = AddedBinding;
		FPropertyBindingDataView SourceDataView;
		if (PropertyBindingsOwner->GetBindingDataViewByID(Binding.GetSourcePath().GetStructID(), SourceDataView))
		{
			Binding.GetMutableSourcePath().UpdateSegmentsFromValue(SourceDataView);
		}
		FPropertyBindingDataView TargetDataView;
		if (PropertyBindingsOwner->GetBindingDataViewByID(Binding.GetTargetPath().GetStructID(), TargetDataView))
		{
			Binding.GetMutableTargetPath().UpdateSegmentsFromValue(TargetDataView);
		}
	}
}

bool FPropertyBindingBindingCollection::HasBinding(const FPropertyBindingPath& InTargetPath, ESearchMode InSearchMode) const
{
	if (InSearchMode == ESearchMode::Exact)
	{
		return HasBindingInternal([&InTargetPath](const FPropertyBindingBinding& Binding)
			{
				return Binding.GetTargetPath() == InTargetPath;
			});
	}
	else
	{
		return HasBindingInternal([&InTargetPath](const FPropertyBindingBinding& Binding)
			{
				return Binding.GetTargetPath().Includes(InTargetPath);
			});
	}
}

const FPropertyBindingBinding* FPropertyBindingBindingCollection::FindBinding(const FPropertyBindingPath& InTargetPath, ESearchMode InSearchMode) const
{
	if (InSearchMode == ESearchMode::Exact)
	{
		return FindBindingInternal([&InTargetPath](const FPropertyBindingBinding& Binding)
			{
				return Binding.GetTargetPath() == InTargetPath;
			});
	}
	else
	{
		return FindBindingInternal([&InTargetPath](const FPropertyBindingBinding& Binding)
			{
				return Binding.GetTargetPath().Includes(InTargetPath);
			});
	}
}

const FPropertyBindingPath* FPropertyBindingBindingCollection::GetBindingSource(const FPropertyBindingPath& InTargetPath) const
{
	const FPropertyBindingBinding* Binding = FindBindingInternal([&InTargetPath](const FPropertyBindingBinding& Binding)
		{
			return Binding.GetTargetPath() == InTargetPath;
		});
	return Binding ? &Binding->GetSourcePath() : nullptr;
}

void FPropertyBindingBindingCollection::GetBindingsFor(const FGuid InStructID, TArray<const FPropertyBindingBinding*>& OutBindings) const
{
	ForEachBinding([&OutBindings, &InStructID](const FPropertyBindingBinding& Binding)
	{
		if (Binding.GetSourcePath().GetStructID().IsValid() && Binding.GetTargetPath().GetStructID() == InStructID)
		{
			OutBindings.Add(&Binding);
		}
	});
}

void FPropertyBindingBindingCollection::RemoveInvalidBindings(const TMap<FGuid, const FPropertyBindingDataView>& InValidStructs, EInvalidBindingCriteria InCriteria)
{
	const UObject* LogOwner = GetLogOwner();
	// first pass, we remove invalid ones
	RemoveBindingsInternal([LogOwner, InValidStructs, InCriteria](const FPropertyBindingBinding& Binding)
		{
			constexpr bool bHandleRedirects = true;

			{
				const FPropertyBindingDataView* SourceValue = InValidStructs.Find(Binding.GetSourcePath().GetStructID());
				if (EnumHasAnyFlags(InCriteria, EInvalidBindingCriteria::RemoveMissingSourceStruct) && (!SourceValue || !SourceValue->IsValid()))
				{
					return true;
				}

				if (EnumHasAnyFlags(InCriteria, EInvalidBindingCriteria::RemoveFailedToResolveSourcePath) && SourceValue && SourceValue->IsValid())
				{
					FString Error;
					TArray<FPropertyBindingPathIndirection> Indirections;
					if (!Binding.GetSourcePath().ResolveIndirectionsWithValue(*SourceValue, Indirections, &Error, bHandleRedirects))
					{
						PROPERTY_BINDINGS_LOG(LogOwner, Log, TEXT("Removing binding from '%s' because source path '%s' cannot be resolved: %s"),
							*GetPathNameSafe(LogOwner), *Binding.GetSourcePath().ToString(), *Error);

						// Remove
						return true;
					}
				}
			}
			
			{
				const FPropertyBindingDataView* TargetValue = InValidStructs.Find(Binding.GetTargetPath().GetStructID());
				if (EnumHasAnyFlags(InCriteria, EInvalidBindingCriteria::RemoveMissingTargetStruct) &&(!TargetValue || !TargetValue->IsValid()))
				{
					return true;
				}

				if (EnumHasAnyFlags(InCriteria, EInvalidBindingCriteria::RemoveFailedToResolveTargetPath) && TargetValue && TargetValue->IsValid())
				{
					// Target path should always have at least one segment (copy bind directly on a target struct/object is not allowed). 
					if (Binding.GetTargetPath().IsPathEmpty())
					{
						return true;
					}

					FString Error;
					TArray<FPropertyBindingPathIndirection> Indirections;
					if (!Binding.GetTargetPath().ResolveIndirectionsWithValue(*TargetValue, Indirections, &Error, bHandleRedirects))
					{
						PROPERTY_BINDINGS_LOG(LogOwner, Log, TEXT("Removing binding from '%s' because target path '%s' cannot be resolved: %s"),
							*GetPathNameSafe(LogOwner), *Binding.GetTargetPath().ToString(), *Error);

						// Remove
						return true;
					}
				}
			}

			return false;
		});

	auto LogRemoveIdenticalOrIncludedBinding = [LogOwner](const FPropertyBindingPath& InTargetBindingPathToRemove, const FPropertyBindingPath& InTargetBindingPathToRetain)
		{
			if (InTargetBindingPathToRemove == InTargetBindingPathToRetain)
			{
				PROPERTY_BINDINGS_LOG(LogOwner, Log, TEXT("Removing binding from '%s' because target path '%s' already has a binding"),
					*GetPathNameSafe(LogOwner),*InTargetBindingPathToRemove.ToString());
			}
			else
			{
				PROPERTY_BINDINGS_LOG(LogOwner, Log, TEXT("Removing binding from '%s' because target path '%s' is under another target path '%s'"),
					*GetPathNameSafe(LogOwner), *InTargetBindingPathToRemove.ToString(), *InTargetBindingPathToRetain.ToString()); 
			}
		};
	// second pass, we remove identical and bindings which have a "parent binding"(Foo.A and Foo, will only keep Foo)
	TMap<FGuid, const FPropertyBindingDataView>::TConstIterator MapIt = InValidStructs.CreateConstIterator();
	while (MapIt)
	{
		TArray<const FPropertyBindingBinding*> StructBindings;
		GetBindingsFor(MapIt.Key(), StructBindings);

		TArray<FPropertyBindingPath, TInlineAllocator<4>> BindingPathsToRemove;

		for (int32 Idx = 0; Idx < StructBindings.Num(); )
		{
			const FPropertyBindingPath& CurrentBindingTargetPath = StructBindings[Idx]->GetTargetPath();
			bool bCurrentBindingRemoved = false;

			for (int32 OtherIdx = Idx + 1; OtherIdx < StructBindings.Num(); )
			{
				const FPropertyBindingPath& BindingToCompareTargetPath = StructBindings[OtherIdx]->GetTargetPath();

				// If the other path is identical or includes the current path, we only keep the shorter one(copying the whole struct instead of copying one of its properties)
				if (BindingToCompareTargetPath.Includes(CurrentBindingTargetPath))
				{
					LogRemoveIdenticalOrIncludedBinding(BindingToCompareTargetPath, CurrentBindingTargetPath);

					BindingPathsToRemove.Add(BindingToCompareTargetPath);
					StructBindings.RemoveAtSwap(OtherIdx);

					continue;
				}

				if (CurrentBindingTargetPath.Includes(BindingToCompareTargetPath))
				{
					LogRemoveIdenticalOrIncludedBinding(CurrentBindingTargetPath, BindingToCompareTargetPath);

					BindingPathsToRemove.Add(CurrentBindingTargetPath);
					StructBindings.RemoveAtSwap(Idx);

					bCurrentBindingRemoved = true;
					break;
				}

				++OtherIdx;
			}

			if (!bCurrentBindingRemoved)
			{
				++Idx;
			}
		}

		RemoveBindingsInternal([&BindingPathsToRemove](FPropertyBindingBinding& InBinding)
		{
			int32 FoundIndex = BindingPathsToRemove.Find(InBinding.GetTargetPath());
			if (FoundIndex == INDEX_NONE)
			{
				return false;
			}

			BindingPathsToRemove.RemoveAtSwap(FoundIndex);
			return true;
		});

		++MapIt;
	}
}
#endif //WITH_EDITOR

void FPropertyBindingBindingCollection::Reset()
{
	CopyBatches.Reset();
	PropertyCopies.Reset();
	PropertyIndirections.Reset();
	
	bBindingsResolved = false;

	OnReset();
}

bool FPropertyBindingBindingCollection::ResolvePaths()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPropertyBindingBindingCollection::ResolvePaths)

	PropertyIndirections.Reset();
	PropertyCopies.SetNum(GetNumBindings());

	bBindingsResolved = true;

	bool bResult = true;
	
	for (const FPropertyBindingCopyInfoBatch& Batch : CopyBatches)
	{
		ForEachBinding(Batch.BindingsBegin, Batch.BindingsEnd, [&bResult, &Batch, this, LogOwner = GetLogOwner()](const FPropertyBindingBinding& Binding, const int32 BindingIndex)
		{
			FPropertyBindingCopyInfo& Copy = PropertyCopies[BindingIndex];
			Copy.SourceDataHandle = Binding.GetSourceDataHandleStruct();

			if (!Copy.SourceDataHandle.IsValid())
			{
				PROPERTY_BINDINGS_LOG(LogOwner, Error
					, TEXT("ResolvePaths failed: Invalid source struct for property binding '%s'."), *Binding.GetSourcePath().ToString());
				Copy.Type = EPropertyCopyType::None;
				bBindingsResolved = false;
				bResult = false;
				return;
			}

			const FPropertyBindingBindableStructDescriptor* SourceDesc = GetBindableStructDescriptorFromHandle(Copy.SourceDataHandle);
			if (!SourceDesc)
			{
				PROPERTY_BINDINGS_LOG(LogOwner, Error
					, TEXT("ResolvePaths failed: Could not find bindable struct descriptor for path '%s'."), *Binding.GetSourcePath().ToString());
				Copy.Type = EPropertyCopyType::None;
				bBindingsResolved = false;
				bResult = false;
				return;
			}

			const UStruct* SourceStruct = SourceDesc->Struct;
			const UStruct* TargetStruct = Batch.TargetStruct.Get().Struct;
			if (!SourceStruct || !TargetStruct)
			{
				PROPERTY_BINDINGS_CLOG(!SourceStruct, LogOwner, Error
					, TEXT("ResolvePaths failed: Could not find source struct for descriptor '%s'."), *SourceDesc->ToString());
				PROPERTY_BINDINGS_CLOG(!TargetStruct, LogOwner,Error
					, TEXT("ResolvePaths failed: Could not find target struct for descriptor '%s'."), *Batch.TargetStruct.Get().ToString());
				Copy.Type = EPropertyCopyType::None;
				bBindingsResolved = false;
				bResult = false;
				return;
			}

			Copy.SourceStructType = SourceStruct;

			// Resolve paths and validate the copy. Stops on first failure.
			bool bSuccess = true;
			FPropertyBindingDataView SourceDataView(SourceStruct, nullptr);
			FPropertyBindingDataView TargetDataView(TargetStruct, nullptr);

			if (IPropertyBindingBindingCollectionOwner* BindingsOwnerInterface = GetBindingsOwner())
			{
				if (!BindingsOwnerInterface->GetBindingDataView(Binding, IPropertyBindingBindingCollectionOwner::EBindingSide::Source, SourceDataView))
				{
					PROPERTY_BINDINGS_LOG(LogOwner, Error, TEXT("ResolvePaths failed: Could not retrieve source data view for '%s'."), *Binding.ToString());
					bSuccess = false;
				}
				if (!BindingsOwnerInterface->GetBindingDataView(Binding, IPropertyBindingBindingCollectionOwner::EBindingSide::Target, TargetDataView))
				{
					PROPERTY_BINDINGS_LOG(LogOwner, Error, TEXT("ResolvePaths failed: Could not retrieve target data view for '%s'."), *Binding.ToString());
					bSuccess = false;
				}
			}

			FPropertyBindingPathIndirection SourceLeafIndirection;
			FPropertyBindingPathIndirection TargetLeafIndirection;

			if (!ResolvePath(SourceDataView, Binding.GetSourcePath(), Copy.SourceIndirection, SourceLeafIndirection))
			{
				PROPERTY_BINDINGS_LOG(LogOwner, Error, TEXT("ResolvePaths failed: Could not resolve source path '%s'."), *Binding.GetSourcePath().ToString());
				bSuccess = false;
			}

			if (!ResolvePath(TargetDataView, Binding.GetTargetPath(), Copy.TargetIndirection, TargetLeafIndirection))
			{
				PROPERTY_BINDINGS_LOG(LogOwner, Error, TEXT("ResolvePaths failed: Could not resolve target path '%s'."), *Binding.GetTargetPath().ToString());
				bSuccess = false;
			}

			if (!ResolveBindingCopyInfo(Binding, SourceLeafIndirection, TargetLeafIndirection, Copy))
			{
				PROPERTY_BINDINGS_LOG(LogOwner, Error, TEXT("ResolvePaths failed: Could not resolve copy info."));
				bSuccess = false;
			}

			if (!bSuccess)
			{
				// Resolving or validating failed, make the copy a nop.
				Copy.Type = EPropertyCopyType::None;
				bResult = false;
			}
			else
			{
				PROPERTY_BINDINGS_LOG(LogOwner, Log, TEXT("ResolvePaths succeeded for '%s'."), *Binding.ToString());
			}
		});
	}

	if (!OnResolvingPaths())
	{
		bResult = false;
	}

	return bResult;
}

bool FPropertyBindingBindingCollection::ResolvePath(const UStruct* Struct, const FPropertyBindingPath& Path,
	FPropertyBindingPropertyIndirection& OutFirstIndirection, FPropertyBindingPathIndirection& OutLeafIndirection)
{
	return ResolvePath(FPropertyBindingDataView{Struct, nullptr}, Path, OutFirstIndirection, OutLeafIndirection);
}

bool FPropertyBindingBindingCollection::ResolvePath(const FPropertyBindingDataView DataView, const FPropertyBindingPath& Path, FPropertyBindingPropertyIndirection& OutFirstIndirection, FPropertyBindingPathIndirection& OutLeafIndirection)
{
	// To preserve legacy behavior we only validate struct and not the whole view using DataView.IsValid() (which also requires valid memory) 
	if (DataView.GetStruct() == nullptr)
 	{
		PROPERTY_BINDINGS_LOG(GetLogOwner(), Error, TEXT("%hs: '%s' Invalid source data view."), __FUNCTION__, *Path.ToString());
		return false;
	}

	FString Error;
	TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>> PathIndirections;
	if (!Path.ResolveIndirectionsWithValue(DataView, PathIndirections, &Error))
	{
		PROPERTY_BINDINGS_LOG(GetLogOwner(), Error, TEXT("%hs: %s"), __FUNCTION__, *Error);
		return false;
	}

	TArray<FPropertyBindingPropertyIndirection, TInlineAllocator<16>> TempIndirections;
	for (FPropertyBindingPathIndirection& PathIndirection : PathIndirections)
	{
		FPropertyBindingPropertyIndirection& Indirection = TempIndirections.AddDefaulted_GetRef();

		check(PathIndirection.GetPropertyOffset() >= MIN_uint16 && PathIndirection.GetPropertyOffset() <= MAX_uint16);

		Indirection.Offset = static_cast<uint16>(PathIndirection.GetPropertyOffset());
		Indirection.Type = PathIndirection.GetAccessType();

		if (Indirection.Type == EPropertyBindingPropertyAccessType::IndexArray)
		{
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(PathIndirection.GetProperty()))
			{
				Indirection.ArrayProperty = ArrayProperty;
				Indirection.ArrayIndex = FPropertyBindingIndex16(PathIndirection.GetArrayIndex());
				if (!Indirection.ArrayIndex.IsValid())
				{
					PROPERTY_BINDINGS_LOG(GetLogOwner(), Error, TEXT("%hs: Array index %d at '%s', is too large."),
						__FUNCTION__, PathIndirection.GetArrayIndex(), *Path.ToString(PathIndirection.GetPathSegmentIndex(), TEXT("<"), TEXT(">")));
					return false;
				}
			}
			else
			{
				PROPERTY_BINDINGS_LOG(GetLogOwner(), Error, TEXT("%hs: Expect property '%s' to be array property."),
					__FUNCTION__, *Path.ToString(PathIndirection.GetPathSegmentIndex(), TEXT("<"), TEXT(">")));
				return false;
			}
		}
		else if (Indirection.Type == EPropertyBindingPropertyAccessType::StructInstance
				|| Indirection.Type == EPropertyBindingPropertyAccessType::SharedStruct
				|| Indirection.Type == EPropertyBindingPropertyAccessType::ObjectInstance
				|| Indirection.Type == EPropertyBindingPropertyAccessType::StructInstanceContainer)
		{
			if (PathIndirection.GetInstanceStruct())
			{
				Indirection.InstanceStruct = PathIndirection.GetInstanceStruct();
				Indirection.ArrayIndex = FPropertyBindingIndex16(PathIndirection.GetArrayIndex());
			}
			else
			{
				PROPERTY_BINDINGS_LOG(GetLogOwner(), Error, TEXT("%hs: Expect instanced property access '%s' to have instance type specified."),
					__FUNCTION__, *Path.ToString(PathIndirection.GetPathSegmentIndex(), TEXT("<"), TEXT(">")));
				return false;
			}
		}
	}

	if (TempIndirections.Num() > 0)
	{
		for (int32 Index = 0; Index < TempIndirections.Num(); Index++)
		{
			FPropertyBindingPropertyIndirection& Indirection = TempIndirections[Index];
			if ((Index + 1) < TempIndirections.Num())
			{
				const FPropertyBindingPropertyIndirection& NextIndirection = TempIndirections[Index + 1];
				if (Indirection.Type == EPropertyBindingPropertyAccessType::Offset
					&& NextIndirection.Type == EPropertyBindingPropertyAccessType::Offset)
				{
					// Collapse adjacent offset indirections
					Indirection.Offset += NextIndirection.Offset;
					TempIndirections.RemoveAt(Index + 1);
					Index--;
				}
				else if (Indirection.Type == EPropertyBindingPropertyAccessType::IndexArray
					&& NextIndirection.Type == EPropertyBindingPropertyAccessType::Offset
					&& NextIndirection.Offset == 0)
				{
					// Remove empty offset after array indexing.
					TempIndirections.RemoveAt(Index + 1);
					Index--;
				}
				else if (Indirection.Type == EPropertyBindingPropertyAccessType::StructInstance
					&& NextIndirection.Type == EPropertyBindingPropertyAccessType::Offset
					&& NextIndirection.Offset == 0)
				{
					// Remove empty offset after struct indirection.
					TempIndirections.RemoveAt(Index + 1);
					Index--;
				}
				else if (Indirection.Type == EPropertyBindingPropertyAccessType::SharedStruct
					&& NextIndirection.Type == EPropertyBindingPropertyAccessType::Offset
					&& NextIndirection.Offset == 0)
				{
					// Remove empty offset after struct indirection.
					TempIndirections.RemoveAt(Index + 1);
					Index--;
				}
				else if (Indirection.Type == EPropertyBindingPropertyAccessType::StructInstanceContainer
					&& NextIndirection.Type == EPropertyBindingPropertyAccessType::Offset
					&& NextIndirection.Offset == 0)
				{
					// Remove empty offset after container indirection.
					TempIndirections.RemoveAt(Index + 1);
					Index--;
				}
				else if ((Indirection.Type == EPropertyBindingPropertyAccessType::Object
						|| Indirection.Type == EPropertyBindingPropertyAccessType::ObjectInstance)
					&& NextIndirection.Type == EPropertyBindingPropertyAccessType::Offset
					&& NextIndirection.Offset == 0)
				{
					// Remove empty offset after object indirection.
					TempIndirections.RemoveAt(Index + 1);
					Index--;
				}
			}
		}

		OutLeafIndirection = PathIndirections.Last(); 

		// Store indirections
		OutFirstIndirection = TempIndirections[0];
		FPropertyBindingPropertyIndirection* PrevIndirection = &OutFirstIndirection;
		for (int32 Index = 1; Index < TempIndirections.Num(); Index++)
		{
			const int32 IndirectionIndex = PropertyIndirections.Num();
			PrevIndirection->NextIndex = FPropertyBindingIndex16(IndirectionIndex); // Set PrevIndirection before array add, as it can invalidate the pointer.
			FPropertyBindingPropertyIndirection& NewIndirection = PropertyIndirections.Add_GetRef(TempIndirections[Index]);
			PrevIndirection = &NewIndirection;
		}
	}
	else
	{
		// Indirections can be empty in case we're directly binding to source structs.
		// Zero offset will return the struct itself.
		OutFirstIndirection.Offset = 0;
		OutFirstIndirection.Type = EPropertyBindingPropertyAccessType::Offset;

		OutLeafIndirection = FPropertyBindingPathIndirection(DataView.GetStruct());
	}

	return true;
}

bool FPropertyBindingBindingCollection::ResolveCopyInfoBetweenIndirections(const FPropertyBindingPathIndirection& InSourceIndirection, const FPropertyBindingPathIndirection& InTargetIndirection, FPropertyBindingCopyInfo& OutCopyInfo) const
{
	// @todo: see if GetPropertyCompatibility() can be implemented as call to ResolveCopyInfoBetweenIndirections() instead so that we write this logic just once.
	
	const FProperty* SourceProperty = InSourceIndirection.GetProperty();
	const UStruct* SourceStruct = InSourceIndirection.GetContainerStruct();
	
	const FProperty* TargetProperty = InTargetIndirection.GetProperty();
	const UStruct* TargetStruct = InTargetIndirection.GetContainerStruct();

	if (!SourceStruct || !TargetStruct)
	{
		return false;
	}

	OutCopyInfo.SourceLeafProperty = InSourceIndirection.GetProperty();
	OutCopyInfo.TargetLeafProperty = InTargetIndirection.GetProperty();

	OutCopyInfo.CopySize = 0;
	OutCopyInfo.Type = EPropertyCopyType::None;

	// Flip Copy source and copy target if the copy direction is reversed
	const FProperty* CopySourceProperty = SourceProperty;
	const UStruct* CopySourceStruct = InSourceIndirection.GetContainerStruct();
	const FProperty* CopyTargetProperty = TargetProperty;
	const UStruct* CopyTargetStruct = InTargetIndirection.GetContainerStruct();
	if (OutCopyInfo.bCopyFromTargetToSource)
	{
		Swap(CopySourceProperty, CopyTargetProperty);
		Swap(CopySourceStruct, CopyTargetStruct);
	}

	if (CopySourceProperty == nullptr)
	{
		// Copy directly from the source struct, target must be the same struct type.
		if (const FStructProperty* TargetStructProperty = CastField<FStructProperty>(CopyTargetProperty))
		{
			if (TargetStructProperty->Struct == CopySourceStruct)
			{
				OutCopyInfo.Type = EPropertyCopyType::CopyStruct;
				return true;
			}
		}
		else if (const FObjectPropertyBase* TargetObjectProperty = CastField<FObjectPropertyBase>(CopyTargetProperty))
		{
			if (CopySourceStruct->IsChildOf(TargetObjectProperty->PropertyClass))
			{
				OutCopyInfo.Type = EPropertyCopyType::CopyObject;
				return true;
			}
		}

		return false;
	}

	// Allow derived classes to support StructReferences
	// StructRef doesn't support copy other way because target may outlive source
	if (PropertyReferenceStructType != nullptr && !OutCopyInfo.bCopyFromTargetToSource)
	{
		if (const FStructProperty* TargetStructProperty = CastField<const FStructProperty>(CopyTargetProperty))
		{
			if (TargetStructProperty->Struct == PropertyReferenceStructType)
			{
				if (const FStructProperty* CopySourceStructProperty = CastField<const FStructProperty>(CopySourceProperty))
				{
					// 'StructReferenceType' to 'StructReferenceType' is copied as normal properties.
					if (CopySourceStructProperty->Struct != PropertyReferenceStructType)
					{
						OutCopyInfo.Type = EPropertyCopyType::StructReference;
						return true;
					}
				}
			}
		}
	}

	UE::PropertyBinding::EPropertyCompatibility Compatibility = UE::PropertyBinding::GetPropertyCompatibility(CopySourceProperty, CopyTargetProperty);

	if (Compatibility == UE::PropertyBinding::EPropertyCompatibility::Promotable)
	{
		const FArrayProperty* CopyTargetArrayProperty = CastField<FArrayProperty>(CopyTargetProperty);
		const FArrayProperty* CopySourceArrayProperty = CastField<FArrayProperty>(CopySourceProperty);
		if (CopyTargetArrayProperty && CopySourceArrayProperty)
		{
			OutCopyInfo.bPromoteArrayCopy = true;
			CopyTargetProperty = CopyTargetArrayProperty->Inner;
			CopySourceProperty = CopySourceArrayProperty->Inner;
			Compatibility = UE::PropertyBinding::GetPropertyCompatibility(CopySourceProperty, CopyTargetProperty);
		}
	}

	// Extract underlying types for enums
	if (const FEnumProperty* EnumPropertyA = CastField<const FEnumProperty>(CopySourceProperty))
	{
		CopySourceProperty = EnumPropertyA->GetUnderlyingProperty();
	}

	if (const FEnumProperty* EnumPropertyB = CastField<const FEnumProperty>(CopyTargetProperty))
	{
		CopyTargetProperty = EnumPropertyB->GetUnderlyingProperty();
	}

	if (Compatibility == UE::PropertyBinding::EPropertyCompatibility::Compatible)
	{
		if (CastField<FNameProperty>(CopyTargetProperty))
		{
			OutCopyInfo.Type = EPropertyCopyType::CopyName;
			return true;
		}
		else if (CastField<FBoolProperty>(CopyTargetProperty))
		{
			OutCopyInfo.Type = EPropertyCopyType::CopyBool;
			return true;
		}
		else if (CastField<FStructProperty>(CopyTargetProperty))
		{
			OutCopyInfo.Type = EPropertyCopyType::CopyStruct;
			return true;
		}
		else if (CastField<FObjectPropertyBase>(CopyTargetProperty))
		{
			const bool bBothAreSoftObjectProperty = CopySourceProperty->IsA<FSoftObjectProperty>() && CopyTargetProperty->IsA<FSoftObjectProperty>();
			const bool bBothAreWeakObjectProperty = CopySourceProperty->IsA<FWeakObjectProperty>() && CopyTargetProperty->IsA<FWeakObjectProperty>();
			const bool bBothAreLazyObjectProperty = CopySourceProperty->IsA<FLazyObjectProperty>() && CopyTargetProperty->IsA<FLazyObjectProperty>();
			if (bBothAreSoftObjectProperty || bBothAreWeakObjectProperty || bBothAreLazyObjectProperty)
			{
				// Use CopyComplex when copying soft object to another soft object so that we do not try to dereference the object (just copies the path).
				// This handles soft class too.
				OutCopyInfo.Type = EPropertyCopyType::CopyComplex;
			}
			else
			{
				OutCopyInfo.Type = EPropertyCopyType::CopyObject;
			}
			return true;
		}
		else if (CastField<FArrayProperty>(CopyTargetProperty) && CopyTargetProperty->HasAnyPropertyFlags(CPF_EditFixedSize))
		{
			// only apply array copying rules if the destination array is fixed size, otherwise it will be 'complex'
			OutCopyInfo.Type = EPropertyCopyType::CopyFixedArray;
			return true;
		}
		else if (CopyTargetProperty->PropertyFlags & CPF_IsPlainOldData)
		{
			OutCopyInfo.Type = EPropertyCopyType::CopyPlain;
			OutCopyInfo.CopySize = CopySourceProperty->GetElementSize() * CopySourceProperty->ArrayDim;
			return true;
		}
		else
		{
			OutCopyInfo.Type = EPropertyCopyType::CopyComplex;
			return true;
		}
	}
	else if (Compatibility == UE::PropertyBinding::EPropertyCompatibility::Promotable)
	{
		if (CopySourceProperty->IsA<FBoolProperty>())
		{
			if (CopyTargetProperty->IsA<FByteProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteBoolToByte;
				return true;
			}
			else if (CopyTargetProperty->IsA<FIntProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteBoolToInt32;
				return true;
			}
			else if (CopyTargetProperty->IsA<FUInt32Property>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteBoolToUInt32;
				return true;
			}
			else if (CopyTargetProperty->IsA<FInt64Property>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteBoolToInt64;
				return true;
			}
			else if (CopyTargetProperty->IsA<FFloatProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteBoolToFloat;
				return true;
			}
			else if (CopyTargetProperty->IsA<FDoubleProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteBoolToDouble;
				return true;
			}
		}
		else if (CopySourceProperty->IsA<FByteProperty>())
		{
			if (CopyTargetProperty->IsA<FIntProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteByteToInt32;
				return true;
			}
			else if (CopyTargetProperty->IsA<FUInt32Property>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteByteToUInt32;
				return true;
			}
			else if (CopyTargetProperty->IsA<FInt64Property>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteByteToInt64;
				return true;
			}
			else if (CopyTargetProperty->IsA<FFloatProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteByteToFloat;
				return true;
			}
			else if (CopyTargetProperty->IsA<FDoubleProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteByteToDouble;
				return true;
			}
		}
		else if (CopySourceProperty->IsA<FIntProperty>())
		{
			if (CopyTargetProperty->IsA<FInt64Property>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteInt32ToInt64;
				return true;
			}
			else if (CopyTargetProperty->IsA<FFloatProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteInt32ToFloat;
				return true;
			}
			else if (CopyTargetProperty->IsA<FDoubleProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteInt32ToDouble;
				return true;
			}
		}
		else if (CopySourceProperty->IsA<FUInt32Property>())
		{
			if (CopyTargetProperty->IsA<FInt64Property>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteUInt32ToInt64;
				return true;
			}
			else if (CopyTargetProperty->IsA<FFloatProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteUInt32ToFloat;
				return true;
			}
			else if (CopyTargetProperty->IsA<FDoubleProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteUInt32ToDouble;
				return true;
			}
		}
		else if (CopySourceProperty->IsA<FFloatProperty>())
		{
			if (CopyTargetProperty->IsA<FIntProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteFloatToInt32;
				return true;
			}
			else if (CopyTargetProperty->IsA<FInt64Property>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteFloatToInt64;
				return true;
			}
			else if (CopyTargetProperty->IsA<FDoubleProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::PromoteFloatToDouble;
				return true;
			}
		}
		else if (CopySourceProperty->IsA<FDoubleProperty>())
		{
			if (CopyTargetProperty->IsA<FIntProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::DemoteDoubleToInt32;
				return true;
			}
			else if (CopyTargetProperty->IsA<FInt64Property>())
			{
				OutCopyInfo.Type = EPropertyCopyType::DemoteDoubleToInt64;
				return true;
			}
			else if (CopyTargetProperty->IsA<FFloatProperty>())
			{
				OutCopyInfo.Type = EPropertyCopyType::DemoteDoubleToFloat;
				return true;
			}
		}
	}

	return false;
}

uint8* FPropertyBindingBindingCollection::GetAddress(FPropertyBindingDataView InStructView, const FPropertyBindingPropertyIndirection& FirstIndirection, const FProperty* LeafProperty) const
{
	uint8* Address = (uint8*)InStructView.GetMutableMemory();
	if (Address == nullptr)
	{
		// Failed indirection, will be reported by caller.
		return nullptr;
	}

	const FPropertyBindingPropertyIndirection* Indirection = &FirstIndirection;

	while (Indirection != nullptr && Address != nullptr)
	{
		switch (Indirection->Type)
		{
		case EPropertyBindingPropertyAccessType::Offset:
		{
			Address = Address + Indirection->Offset;
			break;
		}
		case EPropertyBindingPropertyAccessType::Object:
		{
			UObject* Object = *reinterpret_cast<UObject**>(Address + Indirection->Offset);
			Address = reinterpret_cast<uint8*>(Object);
			break;
		}
		case EPropertyBindingPropertyAccessType::WeakObject:
		{
			FWeakObjectPtr& WeakObjectPtr = *reinterpret_cast<FWeakObjectPtr*>(Address + Indirection->Offset);
			UObject* Object = WeakObjectPtr.Get();
			Address = reinterpret_cast<uint8*>(Object);
			break;
		}
		case EPropertyBindingPropertyAccessType::SoftObject:
		{
			FSoftObjectPtr& SoftObjectPtr = *reinterpret_cast<FSoftObjectPtr*>(Address + Indirection->Offset);
			UObject* Object = SoftObjectPtr.Get();
			Address = reinterpret_cast<uint8*>(Object);
			break;
		}
		case EPropertyBindingPropertyAccessType::ObjectInstance:
		{
			check(Indirection->InstanceStruct);
			UObject* Object = *reinterpret_cast<UObject**>(Address + Indirection->Offset);
			if (Object
				&& Object->GetClass()->IsChildOf(Indirection->InstanceStruct))
			{
				Address = reinterpret_cast<uint8*>(Object);
			}
			else
			{
				// Failed indirection, will be reported by caller.
				return nullptr;
			}
			break;
		}
		case EPropertyBindingPropertyAccessType::StructInstance:
		{
			check(Indirection->InstanceStruct);
			FInstancedStruct& InstancedStruct = *reinterpret_cast<FInstancedStruct*>(Address + Indirection->Offset);
			const UScriptStruct* InstanceType = InstancedStruct.GetScriptStruct();
			if (InstanceType != nullptr
				&& InstanceType->IsChildOf(Indirection->InstanceStruct))
			{
				Address = InstancedStruct.GetMutableMemory();
			}
			else
			{
				// Failed indirection, will be reported by caller.
				return nullptr;
			}
			break;
		}
		case EPropertyBindingPropertyAccessType::StructInstanceContainer:
		{
			check(Indirection->InstanceStruct);
			FStructView StructView;
			FInstancedStructContainer& InstancedStructContainer = *reinterpret_cast<FInstancedStructContainer*>(Address + Indirection->Offset);
			StructView = InstancedStructContainer[Indirection->ArrayIndex.AsInt32()];

			if (StructView.IsValid()
				&& StructView.GetScriptStruct()->IsChildOf(Indirection->InstanceStruct))
			{
				Address = StructView.GetMemory();
			}
			else
			{
				// Failed indirection, will be reported by caller.
				return nullptr;
			}
			break;
		}
		case EPropertyBindingPropertyAccessType::SharedStruct:
		{
			check(Indirection->InstanceStruct);
			FSharedStruct& SharedStruct = *reinterpret_cast<FSharedStruct*>(Address + Indirection->Offset);
			const UScriptStruct* InstanceType = SharedStruct.GetScriptStruct();
			if (InstanceType != nullptr
				&& InstanceType->IsChildOf(Indirection->InstanceStruct))
			{
				Address = SharedStruct.GetMemory();
			}
			else
			{
				// Failed indirection, will be reported by caller.
				return nullptr;
			}
			break;
		}
		case EPropertyBindingPropertyAccessType::IndexArray:
		{
			check(Indirection->ArrayProperty);
			FScriptArrayHelper Helper(Indirection->ArrayProperty, Address + Indirection->Offset);
			if (Helper.IsValidIndex(Indirection->ArrayIndex.Get()))
			{
				Address = Helper.GetRawPtr(Indirection->ArrayIndex.Get());
			}
			else
			{
				// Failed indirection, will be reported by caller.
				return nullptr;
			}
			break;
		}
		default:
			ensureMsgf(false, TEXT("FStateTreePropertyBindings::GetAddress: Unhandled indirection type %s for '%s'"),
				*StaticEnum<EPropertyBindingPropertyAccessType>()->GetValueAsString(Indirection->Type), *LeafProperty->GetNameCPP());
		}

		Indirection = Indirection->NextIndex.IsValid() ? &PropertyIndirections[Indirection->NextIndex.Get()] : nullptr;
	}

	return (uint8*)Address;
}

bool FPropertyBindingBindingCollection::PerformCopy(const FPropertyBindingCopyInfo& Copy, uint8* SourceAddress, uint8* TargetAddress) const
{
	const FProperty* SourceLeafProperty = Copy.SourceLeafProperty;
	const FProperty* TargetLeafProperty = Copy.TargetLeafProperty;
	if (Copy.bCopyFromTargetToSource)
	{
		Swap(SourceAddress, TargetAddress);
		Swap(SourceLeafProperty, TargetLeafProperty);
	}

	// 'SourceAddress' can only be nullptr for object copy (e.g., EPropertyCopyType::CopyObject)
	// Otherwise we simply fail the copy since it might be possible to get outdated bindings (e.g., out of bound array index)
	if (SourceAddress == nullptr && Copy.Type != EPropertyCopyType::CopyObject)
	{
		PROPERTY_BINDINGS_LOG(GetLogOwner(), Verbose, TEXT("%hs skipped: invalid source address for copy type '%s'."), __FUNCTION__, *UEnum::GetValueAsString(Copy.Type));
		return false;
	}

	// Target address is always required
	if (TargetAddress == nullptr)
	{
		PROPERTY_BINDINGS_LOG(GetLogOwner(), Verbose, TEXT("%hs skipped: invalid target address for copy type '%s'."), __FUNCTION__, *UEnum::GetValueAsString(Copy.Type));
		return false;
	}

	check(TargetLeafProperty);
	
	auto PerformCopyInternal = [Self = this, &Copy](const FProperty* SourceLeafProperty, uint8* SourceAddress, const FProperty* TargetLeafProperty, uint8* TargetAddress)
		{
			switch (Copy.Type)
			{
			case EPropertyCopyType::CopyPlain:
				FMemory::Memcpy(TargetAddress, SourceAddress, Copy.CopySize);
				break;
			case EPropertyCopyType::CopyComplex:
				TargetLeafProperty->CopyCompleteValue(TargetAddress, SourceAddress);
				break;
			case EPropertyCopyType::CopyBool:
				CastFieldChecked<const FBoolProperty>(TargetLeafProperty)->SetPropertyValue(TargetAddress, CastFieldChecked<const FBoolProperty>(SourceLeafProperty)->GetPropertyValue(SourceAddress));
				break;
			case EPropertyCopyType::CopyStruct:
				// If SourceProperty == nullptr (pointing to the struct source directly), the GetAddress() did the right thing and is pointing to the beginning of the struct. 
				CastFieldChecked<const FStructProperty>(TargetLeafProperty)->Struct->CopyScriptStruct(TargetAddress, SourceAddress);
				break;
			case EPropertyCopyType::CopyObject:
				if (SourceLeafProperty == nullptr || SourceAddress == nullptr)
				{
					// Source is pointing at object directly.
					CastFieldChecked<const FObjectPropertyBase>(TargetLeafProperty)->SetObjectPropertyValue(TargetAddress, reinterpret_cast<UObject*>(SourceAddress));
				}
				else
				{
					CastFieldChecked<const FObjectPropertyBase>(TargetLeafProperty)->SetObjectPropertyValue(TargetAddress, CastFieldChecked<const FObjectPropertyBase>(SourceLeafProperty)->GetObjectPropertyValue(SourceAddress));
				}
				break;
			case EPropertyCopyType::CopyName:
				CastFieldChecked<const FNameProperty>(TargetLeafProperty)->SetPropertyValue(TargetAddress, CastFieldChecked<const FNameProperty>(SourceLeafProperty)->GetPropertyValue(SourceAddress));
				break;
			case EPropertyCopyType::CopyFixedArray:
			{
				// Copy into fixed sized array (EditFixedSize). Resizable arrays are copied as Complex, and regular fixed sizes arrays via the regular copies (dim specifies array size).
				const FArrayProperty* SourceArrayProperty = CastFieldChecked<const FArrayProperty>(SourceLeafProperty);
				const FArrayProperty* TargetArrayProperty = CastFieldChecked<const FArrayProperty>(TargetLeafProperty);
				FScriptArrayHelper SourceArrayHelper(SourceArrayProperty, SourceAddress);
				FScriptArrayHelper TargetArrayHelper(TargetArrayProperty, TargetAddress);
			
				const int32 MinSize = FMath::Min(SourceArrayHelper.Num(), TargetArrayHelper.Num());
				for (int32 ElementIndex = 0; ElementIndex < MinSize; ++ElementIndex)
				{
					TargetArrayProperty->Inner->CopySingleValue(TargetArrayHelper.GetRawPtr(ElementIndex), SourceArrayHelper.GetRawPtr(ElementIndex));
				}
				break;
			}
			case EPropertyCopyType::StructReference:
			{
				checkf(Self->PropertyReferenceCopyFunc, TEXT("Not expecting EPropertyCopyType::StructReference if copy functor was not provided"));
				const FStructProperty* SourceStructProperty = CastFieldChecked<const FStructProperty>(SourceLeafProperty);
				if (ensure(SourceStructProperty))
				{
					Self->PropertyReferenceCopyFunc(*SourceStructProperty, SourceAddress, TargetAddress);
				}
				break;
			}
			// Bool promotions
			case EPropertyCopyType::PromoteBoolToByte:
				*reinterpret_cast<uint8*>(TargetAddress) = (uint8)CastFieldChecked<const FBoolProperty>(SourceLeafProperty)->GetPropertyValue(SourceAddress);
				break;
			case EPropertyCopyType::PromoteBoolToInt32:
				*reinterpret_cast<int32*>(TargetAddress) = (int32)CastFieldChecked<const FBoolProperty>(SourceLeafProperty)->GetPropertyValue(SourceAddress);
				break;
			case EPropertyCopyType::PromoteBoolToUInt32:
				*reinterpret_cast<uint32*>(TargetAddress) = (uint32)CastFieldChecked<const FBoolProperty>(SourceLeafProperty)->GetPropertyValue(SourceAddress);
				break;
			case EPropertyCopyType::PromoteBoolToInt64:
				*reinterpret_cast<int64*>(TargetAddress) = (int64)CastFieldChecked<const FBoolProperty>(SourceLeafProperty)->GetPropertyValue(SourceAddress);
				break;
			case EPropertyCopyType::PromoteBoolToFloat:
				*reinterpret_cast<float*>(TargetAddress) = (float)CastFieldChecked<const FBoolProperty>(SourceLeafProperty)->GetPropertyValue(SourceAddress);
				break;
			case EPropertyCopyType::PromoteBoolToDouble:
				*reinterpret_cast<double*>(TargetAddress) = (double)CastFieldChecked<const FBoolProperty>(SourceLeafProperty)->GetPropertyValue(SourceAddress);
				break;
		
			// Byte promotions	
			case EPropertyCopyType::PromoteByteToInt32:
				*reinterpret_cast<int32*>(TargetAddress) = (int32)*reinterpret_cast<const uint8*>(SourceAddress);
				break;
			case EPropertyCopyType::PromoteByteToUInt32:
				*reinterpret_cast<uint32*>(TargetAddress) = (uint32)*reinterpret_cast<const uint8*>(SourceAddress);
				break;
			case EPropertyCopyType::PromoteByteToInt64:
				*reinterpret_cast<int64*>(TargetAddress) = (int64)*reinterpret_cast<const uint8*>(SourceAddress);
				break;
			case EPropertyCopyType::PromoteByteToFloat:
				*reinterpret_cast<float*>(TargetAddress) = (float)*reinterpret_cast<const uint8*>(SourceAddress);
				break;
			case EPropertyCopyType::PromoteByteToDouble:
				*reinterpret_cast<double*>(TargetAddress) = (double)*reinterpret_cast<const uint8*>(SourceAddress);
				break;

			// Int32 promotions
			case EPropertyCopyType::PromoteInt32ToInt64:
				*reinterpret_cast<int64*>(TargetAddress) = (int64)*reinterpret_cast<const int32*>(SourceAddress);
				break;
			case EPropertyCopyType::PromoteInt32ToFloat:
				*reinterpret_cast<float*>(TargetAddress) = (float)*reinterpret_cast<const int32*>(SourceAddress);
				break;
			case EPropertyCopyType::PromoteInt32ToDouble:
				*reinterpret_cast<double*>(TargetAddress) = (double)*reinterpret_cast<const int32*>(SourceAddress);
				break;

			// Uint32 promotions
			case EPropertyCopyType::PromoteUInt32ToInt64:
				*reinterpret_cast<int64*>(TargetAddress) = (int64)*reinterpret_cast<const uint32*>(SourceAddress);
				break;
			case EPropertyCopyType::PromoteUInt32ToFloat:
				*reinterpret_cast<float*>(TargetAddress) = (float)*reinterpret_cast<const uint32*>(SourceAddress);
				break;
			case EPropertyCopyType::PromoteUInt32ToDouble:
				*reinterpret_cast<double*>(TargetAddress) = (double)*reinterpret_cast<const uint32*>(SourceAddress);
				break;

			// Float promotions
			case EPropertyCopyType::PromoteFloatToInt32:
				*reinterpret_cast<int32*>(TargetAddress) = (int32)*reinterpret_cast<const float*>(SourceAddress);
				break;
			case EPropertyCopyType::PromoteFloatToInt64:
				*reinterpret_cast<int64*>(TargetAddress) = (int64)*reinterpret_cast<const float*>(SourceAddress);
				break;
			case EPropertyCopyType::PromoteFloatToDouble:
				*reinterpret_cast<double*>(TargetAddress) = (double)*reinterpret_cast<const float*>(SourceAddress);
				break;

			// Double promotions
			case EPropertyCopyType::DemoteDoubleToInt32:
				*reinterpret_cast<int32*>(TargetAddress) = (int32)*reinterpret_cast<const double*>(SourceAddress);
				break;
			case EPropertyCopyType::DemoteDoubleToInt64:
				*reinterpret_cast<int64*>(TargetAddress) = (int64)*reinterpret_cast<const double*>(SourceAddress);
				break;
			case EPropertyCopyType::DemoteDoubleToFloat:
				*reinterpret_cast<float*>(TargetAddress) = (float)*reinterpret_cast<const double*>(SourceAddress);
				break;

			default:
				ensureMsgf(false, TEXT("FStateTreePropertyBindings::PerformCopy: Unhandled copy type %s between '%s' and '%s'"),
					*StaticEnum<EPropertyCopyType>()->GetValueAsString(Copy.Type), *SourceLeafProperty->GetNameCPP(), *TargetLeafProperty->GetNameCPP());
				break;
			}
		};

	if (!Copy.bPromoteArrayCopy)
	{
		PerformCopyInternal(SourceLeafProperty, SourceAddress, TargetLeafProperty, TargetAddress);
	}
	else
	{
		const FArrayProperty* SourceArrayProperty = CastFieldChecked<const FArrayProperty>(SourceLeafProperty);
		const FArrayProperty* TargetArrayProperty = CastFieldChecked<const FArrayProperty>(TargetLeafProperty);
		FScriptArrayHelper SourceArrayHelper(SourceArrayProperty, SourceAddress);
		FScriptArrayHelper TargetArrayHelper(TargetArrayProperty, TargetAddress);

		const int32 NumberOfElements = SourceArrayHelper.Num();
		if (TargetArrayProperty->HasAnyPropertyFlags(CPF_EditFixedSize) && TargetArrayHelper.Num() != NumberOfElements)
		{
			// The target is of fixed size. The size of the source need to match. It would be bad to copy half or none of the property.
			PROPERTY_BINDINGS_LOG(GetLogOwner(), Verbose, TEXT("%hs skipped: the target array size do not match '%d'."), __FUNCTION__, NumberOfElements);
			return false;
		}

		TargetArrayHelper.Resize(NumberOfElements);
		for (int32 ElementIndex = 0; ElementIndex < NumberOfElements; ++ElementIndex)
		{
			PerformCopyInternal(SourceArrayProperty->Inner, SourceArrayHelper.GetRawPtr(ElementIndex), TargetArrayProperty->Inner, TargetArrayHelper.GetRawPtr(ElementIndex));
		}
	}

	return true;
}

bool FPropertyBindingBindingCollection::CopyProperty(const FPropertyBindingCopyInfo& Copy, FPropertyBindingDataView SourceStructView, FPropertyBindingDataView TargetStructView) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPropertyBindingBindingCollection::CopyProperty)

	// This is made ensure so that the programmers have the chance to catch it (it's usually programming error not to call ResolvePaths(), and it wont spam log for others.
	if (!ensureMsgf(bBindingsResolved, TEXT("Bindings must be resolved successfully before copying. See ResolvePaths()")))
	{
		return false;
	}

	// Copies that fail to be resolved (i.e. property path does not resolve, types changed) will be marked as None, skip them.
	if (Copy.Type == EPropertyCopyType::None)
	{
		return true;
	}

	bool bResult = true;
	
	if (SourceStructView.IsValid() && TargetStructView.IsValid())
	{
		check(SourceStructView.GetStruct() == Copy.SourceStructType
			|| (SourceStructView.GetStruct() && SourceStructView.GetStruct()->IsChildOf(Copy.SourceStructType)));
			
		uint8* SourceAddress = GetAddress(SourceStructView, Copy.SourceIndirection, Copy.SourceLeafProperty);
		uint8* TargetAddress = GetAddress(TargetStructView, Copy.TargetIndirection, Copy.TargetLeafProperty);

		return PerformCopy(Copy, SourceAddress, TargetAddress);
	}
	else
	{
		bResult = false;
	}

	return bResult;
}

void FPropertyBindingBindingCollection::PerformResetObjects(const FPropertyBindingCopyInfo& Copy, uint8* TargetAddress) const
{
	// Source property can be null
	check(Copy.TargetLeafProperty);
	check(TargetAddress);

	auto PerformResetObjectsInternal = [Self = this, Type = Copy.Type](const FProperty* TargetLeafProperty, uint8* TargetAddress)
		{
			switch (Type)
			{
			case EPropertyCopyType::CopyComplex:
				TargetLeafProperty->ClearValue(TargetAddress);
				break;
			case EPropertyCopyType::CopyStruct:
				CastFieldChecked<const FStructProperty>(TargetLeafProperty)->Struct->ClearScriptStruct(TargetAddress);
				break;
			case EPropertyCopyType::CopyObject:
				CastFieldChecked<const FObjectPropertyBase>(TargetLeafProperty)->SetObjectPropertyValue(TargetAddress, nullptr);
				break;
			case EPropertyCopyType::StructReference:
				checkf(Self->PropertyReferenceResetFunc, TEXT("Not expecting EPropertyCopyType::StructReference if reset object functor was not provided"));
				Self->PropertyReferenceResetFunc(TargetAddress);
				break;
			case EPropertyCopyType::CopyFixedArray:
			{
				// Copy into fixed sized array (EditFixedSize). Resizable arrays are copied as Complex, and regular fixed sizes arrays via the regular copies (dim specifies array size).
				const FArrayProperty* TargetArrayProperty = CastFieldChecked<const FArrayProperty>(TargetLeafProperty);
				FScriptArrayHelper TargetArrayHelper(TargetArrayProperty, TargetAddress);
				for (int32 ElementIndex = 0; ElementIndex < TargetArrayHelper.Num(); ++ElementIndex)
				{
					TargetArrayProperty->Inner->ClearValue(TargetArrayHelper.GetRawPtr(ElementIndex));
				}
				break;
			}
			default:
				break;
			}
		};

	if (!Copy.bPromoteArrayCopy)
	{
		PerformResetObjectsInternal(Copy.TargetLeafProperty, TargetAddress);
	}
	else
	{
		const FArrayProperty* TargetArrayProperty = CastFieldChecked<const FArrayProperty>(Copy.TargetLeafProperty);
		FScriptArrayHelper TargetArrayHelper(TargetArrayProperty, TargetAddress);
		const int32 NumberOfElements = TargetArrayHelper.Num();
		for (int32 ElementIndex = 0; ElementIndex < NumberOfElements; ++ElementIndex)
		{
			PerformResetObjectsInternal(TargetArrayProperty->Inner, TargetArrayHelper.GetRawPtr(ElementIndex));
		}

	}
}

bool FPropertyBindingBindingCollection::ResetObjects(const FPropertyBindingIndex16 TargetBatchIndex, FPropertyBindingDataView TargetStructView) const
{
	// This is made ensure so that the programmers have the chance to catch it (it's usually programming error not to call ResolvePaths(), and it wont spam log for others.
	if (!ensureMsgf(bBindingsResolved, TEXT("Bindings must be resolved successfully before copying. See ResolvePaths()")))
	{
		return false;
	}

	if (!TargetBatchIndex.IsValid())
	{
		return false;
	}

	check(CopyBatches.IsValidIndex(TargetBatchIndex.Get()));
	const FPropertyBindingCopyInfoBatch& Batch = CopyBatches[TargetBatchIndex.Get()];

	check(TargetStructView.IsValid());
	check(TargetStructView.GetStruct() == Batch.TargetStruct.Get().Struct);

	for (int32 i = Batch.BindingsBegin.Get(); i != Batch.BindingsEnd.Get(); i++)
	{
		const FPropertyBindingCopyInfo& Copy = PropertyCopies[i];
		// Copies that fail to be resolved (i.e. property path does not resolve, types changed) will be marked as None, skip them.
		if (Copy.Type == EPropertyCopyType::None)
		{
			continue;
		}

		// Validate target address since resetting a previous bindings might invalidate some subsequent bindings
		// targeting inner values (e.g., array got reset and bindings to item no longer need to be reset)
		if (uint8* TargetAddress = GetAddress(TargetStructView, Copy.TargetIndirection, Copy.TargetLeafProperty))
		{
			PerformResetObjects(Copy, TargetAddress);
		}
	}

	return true;
}

bool FPropertyBindingBindingCollection::ContainsAnyStruct(const TSet<const UStruct*>& InStructs) const
{
	// Look in derived classes source struct descriptors
	bool bFoundInSourceStructs = false;
	VisitSourceStructDescriptorInternal([&InStructs, &bFoundInSourceStructs](const FPropertyBindingBindableStructDescriptor& Descriptor)
		{
			if (InStructs.Contains(Descriptor.Struct))
			{
				bFoundInSourceStructs = true;
				return EVisitResult::Break;
			}
			return EVisitResult::Continue;
		});

	if (bFoundInSourceStructs)
	{
		return true;
	}

	for (const FPropertyBindingCopyInfoBatch& CopyBatch : CopyBatches)
	{
		if (InStructs.Contains(CopyBatch.TargetStruct.Get().Struct))
		{
			return true;
		}
	}

	// @todo: at runtime, we won't be able to check any Container Struct.
	// Ex: we are able to check Container Structs for FStateTreeEditorPropertyBindings but not FStateTreePropertyBindings 
	bool bPathContainsStruct = false;

	TArray<const UStruct*> StructsArray;
	StructsArray.Reserve(InStructs.Num());
	Algo::Copy(InStructs, StructsArray, Algo::NoRef);

#if WITH_EDITOR
	if (const IPropertyBindingBindingCollectionOwner* BindingsOwnerPtr = GetBindingsOwner())
	{
		VisitBindings([&bPathContainsStruct, &BindingsOwnerPtr, &StructsArray](const FPropertyBindingBinding& Binding)
			{
				FPropertyBindingDataView SourceDataView;
				BindingsOwnerPtr->GetBindingDataViewByID(Binding.GetSourcePath().GetStructID(), SourceDataView);
				if (Binding.GetSourcePath().ContainsAnyStructWithValue(StructsArray, SourceDataView))
				{
					bPathContainsStruct = true;
					return EVisitResult::Break;
				}

				FPropertyBindingDataView TargetDataView;
				BindingsOwnerPtr->GetBindingDataViewByID(Binding.GetTargetPath().GetStructID(), TargetDataView);
				if (Binding.GetTargetPath().ContainsAnyStructWithValue(StructsArray, TargetDataView))
				{
					bPathContainsStruct = true;
					return EVisitResult::Break;
				}

				return EVisitResult::Continue;
			});
	}
	else
#endif //WITH_EDITOR
	{
		VisitBindings([&bPathContainsStruct, &StructsArray](const FPropertyBindingBinding& Binding)
			{
				if (Binding.GetSourcePath().ContainsAnyStruct(StructsArray))
				{
					bPathContainsStruct = true;
					return EVisitResult::Break;
				}

				if (Binding.GetTargetPath().ContainsAnyStruct(StructsArray))
				{
					bPathContainsStruct = true;
					return EVisitResult::Break;
				}

				return EVisitResult::Continue;
			});
	}

	return bPathContainsStruct;
}

bool FPropertyBindingBindingCollection::ResolveBindingCopyInfo(const FPropertyBindingBinding& InResolvedBinding,
	const FPropertyBindingPathIndirection& InBindingSourceLeafIndirection, const FPropertyBindingPathIndirection& InBindingTargetLeafIndirection,
	FPropertyBindingCopyInfo& OutCopyInfo)
{
	return ResolveCopyInfoBetweenIndirections(InBindingSourceLeafIndirection, InBindingTargetLeafIndirection, OutCopyInfo);
}

#if WITH_EDITOR || WITH_PROPERTYBINDINGUTILS_DEBUG
FString FPropertyBindingBindingCollection::DebugAsString() const
{
	TStringBuilder<0> DebugString;
	/** Array of expected source structs. */
	DebugString.Appendf(TEXT("\nSourceStructs (%d)\n"), GetNumBindableStructDescriptors());
	if (GetNumBindableStructDescriptors())
	{
		DebugString.Appendf(TEXT("  [ (Idx) | %-45s | % -80s ]\n"), TEXT("Type"), TEXT("DataSource"));
		int32 Index = 0;
		VisitSourceStructDescriptorInternal([&Index, &DebugString](const FPropertyBindingBindableStructDescriptor& Descriptor)
		{
			DebugString.Appendf(TEXT("  | (%3d) | %-45s | % -80s |\n"),
				Index,
				*GetNameSafe(Descriptor.Struct),
				*Descriptor.ToString());

			++Index;
			return EVisitResult::Continue;
		});
	}

	/** Array of copy batches. */
	DebugString.Appendf(TEXT("\nCopyBatches (%d)\n"), CopyBatches.Num());
	if (CopyBatches.Num())
	{
		DebugString.Appendf(TEXT("  [ (Idx) | %-45s | %-80s | %-8s [%-5s:%-5s[ | %-8s [%-5s:%-5s[ ]\n"),
			TEXT("Target Type"), TEXT("Target Name"),
			TEXT("Bindings"), TEXT("Beg"), TEXT("End"),
			TEXT("ProFunc"), TEXT("Beg"), TEXT("End"));
		for (int32 Index = 0; Index < CopyBatches.Num(); ++Index)
		{
			const FPropertyBindingCopyInfoBatch& CopyBatch = CopyBatches[Index];
			const FPropertyBindingBindableStructDescriptor& Descriptor = CopyBatch.TargetStruct.Get();
			DebugString.Appendf(TEXT("  | (%3d) | %-45s | %-80s | %8s [%5d:%-5d[ | %8s [%5d:%-5d[ |\n"),
				Index,
				Descriptor.Struct ? *Descriptor.Struct->GetName() : TEXT("null"),
				*Descriptor.ToString(),
				TEXT(""), CopyBatch.BindingsBegin.Get(), CopyBatch.BindingsEnd.Get(),
				TEXT(""), CopyBatch.PropertyFunctionsBegin.Get(), CopyBatch.PropertyFunctionsEnd.Get());
		}
	}

	/** Array of property bindings, resolved into arrays of copies before use. */
	DebugString.Appendf(TEXT("\nPropertyPathBindings (%d)\n"), GetNumBindings());
	if (GetNumBindings())
	{
		DebugString.Appendf(TEXT("  [ (Idx) | %-45s | %-45s | %-45s ]\n"),
			TEXT("Source"), TEXT("SrcPath"), TEXT("TargetPath"));
		int32 Index = 0;
		ForEachBinding([&Index, &DebugString](const FPropertyBindingBinding& PropertyBinding)
		{
			DebugString.Appendf(TEXT("  | (%3d) | %-45s | %-45s |\n"),
				Index,
				*PropertyBinding.GetSourcePath().ToString(),
				*PropertyBinding.GetTargetPath().ToString());
			++Index;
		});
	}

	/** Array of property copies */
	DebugString.Appendf(TEXT("\nPropertyCopies (%d)\n"), PropertyCopies.Num());
	if (PropertyCopies.Num())
	{
		DebugString.Appendf(TEXT("  [ (Idx) | %-7s | %-4s | %-7s | %-10s | %-7s | %-4s | %-7s | %-10s | %-20s | %-4s ]\n"),
			TEXT("Src Idx"), TEXT("Off."), TEXT("Next"), TEXT("Type"),
			TEXT("Tgt Idx"), TEXT("Off."), TEXT("Next"), TEXT("Type"),
			TEXT("Copy Type"), TEXT("Size"));
		for (int32 Index = 0; Index < PropertyCopies.Num(); ++Index)
		{
			const FPropertyBindingCopyInfo& PropertyCopy = PropertyCopies[Index];
			DebugString.Appendf(TEXT("  | (%3d) | %7d | %4d | %7d | %-10s | %7d | %4d | %7d | %-10s | %-20s | %4d |\n"),
						Index,
						PropertyCopy.SourceIndirection.ArrayIndex.Get(),
						PropertyCopy.SourceIndirection.Offset,
						PropertyCopy.SourceIndirection.NextIndex.Get(),
						*UEnum::GetDisplayValueAsText(PropertyCopy.SourceIndirection.Type).ToString(),
						PropertyCopy.TargetIndirection.ArrayIndex.Get(),
						PropertyCopy.TargetIndirection.Offset,
						PropertyCopy.TargetIndirection.NextIndex.Get(),
						*UEnum::GetDisplayValueAsText(PropertyCopy.TargetIndirection.Type).ToString(),
						*UEnum::GetDisplayValueAsText(PropertyCopy.Type).ToString(),
						PropertyCopy.CopySize);
		}
	}

	/** Array of property indirections, indexed by accesses */
	DebugString.Appendf(TEXT("\nPropertyIndirections (%d)\n"), PropertyIndirections.Num());
	if (PropertyIndirections.Num())
	{
		DebugString.Appendf(TEXT("[ (Idx) | %-4s | %-4s | %-4s | %-10s ] \n"),
			TEXT("Idx"), TEXT("Off."), TEXT("Next"), TEXT("Access Type"));
		for (int32 Index = 0; Index < PropertyIndirections.Num(); ++Index)
		{
			const FPropertyBindingPropertyIndirection& PropertyIndirection = PropertyIndirections[Index];
			DebugString.Appendf(TEXT("  | (%3d) | %4d | %4d | %4d | %-10s |\n"),
				Index,
				PropertyIndirection.ArrayIndex.Get(),
				PropertyIndirection.Offset,
				PropertyIndirection.NextIndex.Get(),
				*UEnum::GetDisplayValueAsText(PropertyIndirection.Type).ToString());
		}
	}

	return FString(DebugString);
}
#endif // WITH_EDITOR || WITH_PROPERTYBINDINGUTILS_DEBUG

#undef PROPERTY_BINDINGS_LOG
#undef PROPERTY_BINDINGS_CLOG

