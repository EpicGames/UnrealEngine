// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBindingBindingCollectionOwner.h"
#include "PropertyBindingSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyBindingBindingCollectionOwner)

#if WITH_EDITOR
void IPropertyBindingBindingCollectionOwner::OnPromotingToParameter(FPropertyBagPropertyDesc& PropertyDesc)
{
	// By default, we create a descriptor based on the Target Property, but without the meta-data.
	// This functionality mirrors the user action of adding a new property from the UI, where meta-data is not available.
	// Additionally, meta-data like EditCondition is not desirable here.
	// UPropertyBindingSettings::MetaDataToKeepWhenPromotingToParameter can be modified if a specific metadata needs to stay.
	// Derived classes can override this behavior.
	PropertyDesc.MetaClass = nullptr;
	PropertyDesc.MetaData.RemoveAllSwap(
		[](const FPropertyBagPropertyDescMetaData& MetaData)
		{
			return !GetDefault<UPropertyBindingSettings>()->MetaDataToKeepWhenPromotingToParameter.Contains(MetaData.Key);
		});
}
#endif
