// Copyright Epic Games, Inc. All Rights Reserved.
#include "PropertyBindingPath.h"
#include "PropertyBindingDataView.h"
#include "Misc/EnumerateRange.h"
#include "StructUtils/InstancedStructContainer.h"

#if WITH_EDITOR
#include "UObject/CoreRedirects.h"
#include "UObject/Package.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Kismet2/StructureEditorUtils.h"
#endif


#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyBindingPath)

bool FPropertyBindingPath::FromString(const FStringView InPath)
{
	Segments.Reset();
	
	if (InPath.IsEmpty())
	{
		return true;
	}
	
	auto ParseSegment = [This = this](const FStringView Segment)
	{
		if (Segment.IsEmpty())
		{
			return false;
		}

		int32 FirstBracket = INDEX_NONE;
		int32 LastBracket = INDEX_NONE;
		if (Segment.FindChar(TEXT('['), FirstBracket) && Segment.FindLastChar(TEXT(']'), LastBracket))
		{
			const int32 NameStringLength = FirstBracket;
			const int32 IndexStringLength = LastBracket - FirstBracket - 1;
			if (NameStringLength < 1
				|| IndexStringLength <= 0)
			{
				return false;
			}

			const FStringView NameString = Segment.Left(FirstBracket);
			const FStringView IndexString = Segment.Mid(FirstBracket + 1, IndexStringLength);
			int32 ArrayIndex = INDEX_NONE;
			LexFromString(ArrayIndex, IndexString.GetData());
			if (ArrayIndex < 0)
			{
				return false;
			}
			
			This->AddPathSegment(FName(NameString), ArrayIndex);
		}
		else
		{
			This->AddPathSegment(FName(Segment));
		}
		return true;
	};

	bool bResult = true;
	FStringView Path = InPath;
	int32 FoundChar = INDEX_NONE;
	while (Path.FindChar(TEXT('.'), FoundChar))
	{
		const FStringView Segment = Path.SubStr(0, FoundChar);
		if (!ParseSegment(Segment))
		{
			bResult = false;
			break;
		}
		Path = Path.Mid(FoundChar+1);
	}
	if (bResult)
	{
		bResult = ParseSegment(Path);
	}

	if (!bResult)
	{
		Segments.Reset();
	}
	
	return bResult;
}

bool FPropertyBindingPath::UpdateSegments(const UStruct* BaseStruct, FString* OutError)
{
	return UpdateSegmentsFromValue(FPropertyBindingDataView(BaseStruct, nullptr), OutError);
}

bool FPropertyBindingPath::UpdateSegmentsFromValue(const FPropertyBindingDataView BaseValueView, FString* OutError)
{
	TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>> Indirections;
	if (ResolveIndirectionsWithValue(BaseValueView, Indirections, OutError, /*bHandleRedirects*/true))
	{
		UpdateSegmentsFromIndirections(Indirections);
		return true;
	}
	
	return false;
}

void FPropertyBindingPath::UpdateSegmentsFromIndirections(const TConstArrayView<FPropertyBindingPathIndirection>& Indirections)
{
	for (const FPropertyBindingPathIndirection& Indirection : Indirections)
	{
		FPropertyBindingPathSegment& Segment = Segments[Indirection.PathSegmentIndex];

		if (Indirection.InstanceStruct != nullptr)
		{
			UE_CLOGF(Indirection.InstanceStruct != Segment.GetInstanceStruct(), LogPropertyBindingUtils, Verbose
				, "Updating instanced struct for segment '%ls' in path '%ls' from '%ls' to '%ls'"
				, *Segment.GetName().ToString(), *ToString(), *GetNameSafe(Segment.GetInstanceStruct()), *GetNameSafe(Indirection.InstanceStruct));
			Segment.SetInstanceStruct(Indirection.InstanceStruct, Indirection.GetAccessType());
		}
		else
		{
			UE_CLOGF(Segment.GetInstanceStruct(), LogPropertyBindingUtils, Verbose, "Clearing instanced struct for segment '%ls' in path '%ls'"
				, *Segment.GetName().ToString(), *ToString());
			Segment.SetInstanceStruct(nullptr, EPropertyBindingPropertyAccessType::Unset);
		}
#if WITH_EDITORONLY_DATA
		if (!Indirection.GetRedirectedName().IsNone())
		{
			Segment.SetName(Indirection.GetRedirectedName());
		}
		Segment.SetPropertyGuid(Indirection.GetPropertyGuid());
		if (!Indirection.GetDisplayName().IsNone())
		{
			Segment.SetDisplayName(Indirection.GetDisplayName());
		}
#endif
	}
}

bool FPropertyBindingPath::ContainsAnyStruct(TConstArrayView<const UStruct*> InStructs) const
{
	for (const FPropertyBindingPathSegment& Segment : Segments)
	{
		if (InStructs.Contains(Segment.GetInstanceStruct()))
		{
			return true;
		}
	}

	return false;
}

bool FPropertyBindingPath::ContainsAnyStruct(TConstArrayView<const UStruct*> InStructs, const UStruct* BaseStruct) const
{
	return ContainsAnyStructWithValue(InStructs, FPropertyBindingDataView(BaseStruct, nullptr));
}

bool FPropertyBindingPath::ContainsAnyStructWithValue(TConstArrayView<const UStruct*> InStructs, const FPropertyBindingDataView BaseValueView) const
{
	constexpr bool bHandleRedirects = true;
	constexpr FString* Error = nullptr;
	// Check for any of the structs found so far even if it failed to resolve
	constexpr bool bResetIndirectionsIfFailed = false;

	TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>> Indirections;
	ResolveIndirectionsWithValue(BaseValueView, Indirections, Error, bHandleRedirects, bResetIndirectionsIfFailed);

	return IndirectionsContainAnyStruct(InStructs, Indirections);
}

bool FPropertyBindingPath::IndirectionsContainAnyStruct(TConstArrayView<const UStruct*> InStructs, const TConstArrayView<FPropertyBindingPathIndirection> Indirections)
{
	for (const FPropertyBindingPathIndirection& Indirection : Indirections)
	{
		if (InStructs.Contains(Indirection.GetInstanceStruct()) || InStructs.Contains(Indirection.GetContainerStruct()))
		{
			return true;
		}
	}

	return false;
}

FString FPropertyBindingPath::ToString(const int32 HighlightedSegment, const TCHAR* HighlightPrefix, const TCHAR* HighlightPostfix, const bool bOutputInstances, const int32 FirstSegment) const
{
	TStringBuilder<256> Result;
	for (TEnumerateRef<const FPropertyBindingPathSegment> Segment : EnumerateRange(TConstArrayView<FPropertyBindingPathSegment>(Segments).Mid(FMath::Max(FirstSegment, 0))))
	{
		if (Segment.GetIndex() > 0)
		{
			Result << TEXT('.');
		}
		if (Segment.GetIndex() == HighlightedSegment && HighlightPrefix)
		{
			Result << HighlightPrefix;
		}

		if (bOutputInstances && Segment->GetInstanceStruct())
		{
			Result << TEXT('(');
			Result << Segment->GetInstanceStruct()->GetFName();
			Result << TEXT(')');
		}

#if WITH_EDITORONLY_DATA
		if (!Segment->GetDisplayName().IsNone())
		{
			Result << Segment->GetDisplayName();
		}
		else
#endif // WITH_EDITORONLY_DATA
		{
			Result << Segment->GetName();
		}

		if (Segment->GetArrayIndex() >= 0)
		{
			Result << TEXT('[');
			Result << Segment->GetArrayIndex();
			Result << TEXT(']');
		}

		if (Segment.GetIndex() == HighlightedSegment && HighlightPostfix)
		{
			Result << HighlightPostfix;
		}
	}
	return FString(Result);
}

bool FPropertyBindingPath::Includes(const FPropertyBindingPath& Other) const
{
#if WITH_EDITORONLY_DATA
	if (StructID != Other.StructID)
	{
		return false;
	}
#endif // WITH_EDITORONLY_DATA

	if (Segments.Num() < Other.Segments.Num())
	{
		return false;
	}

	for (TEnumerateRef<const FPropertyBindingPathSegment> OtherSegment : EnumerateRange(Other.Segments))
	{
		const FPropertyBindingPathSegment& Segment = Segments[OtherSegment.GetIndex()];
		if (*OtherSegment == Segment)
		{
			continue;
		}

		// Special case for Array: If Other is bound directly to an array and this is bound to an array element, should be deemed as Inclusion(FooArray[3] should include FooArray)
		// We query and remove bindings based on hierarchy relationship, binding to array should remove any binding to individual array element
		if (OtherSegment.GetIndex() == (Other.Segments.Num() - 1))
		{
			// Bind to array directly will be [Array, Index = -1]. Bind to array element will be [ArrayElement, Index]
			// ArrayElement(Inner) and Array shares the same name
			// we don't allow binding to instanced struct container directly
			if (Segment.GetArrayIndex() != INDEX_NONE && Segment.GetName() == OtherSegment->GetName() && OtherSegment->GetArrayIndex() == INDEX_NONE)
			{
				return true;
			}
		}

		return false;
	}

	return true;
}

bool FPropertyBindingPath::ResolveIndirections(const UStruct* BaseStruct, TArray<FPropertyBindingPathIndirection>& OutIndirections, FString* OutError, bool bHandleRedirects, bool bResetIndirectionsIfFailed) const
{
	return ResolveIndirectionsWithValue(FPropertyBindingDataView(BaseStruct, nullptr), OutIndirections, OutError, bHandleRedirects, bResetIndirectionsIfFailed);
}

bool FPropertyBindingPath::ResolveIndirections(const UStruct* BaseStruct,
	TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>>& OutIndirections, FString* OutError, bool bHandleRedirects, bool bResetIndirectionsIfFailed) const
{
	return ResolveIndirectionsWithValue(FPropertyBindingDataView(BaseStruct, nullptr), OutIndirections, OutError, bHandleRedirects, bResetIndirectionsIfFailed);
}

template<typename Allocator>
bool FPropertyBindingPath::ResolveIndirectionsWithValue(const FPropertyBindingDataView BaseValueView, TArray<FPropertyBindingPathIndirection, Allocator>& OutIndirections, FString* OutError, bool bHandleRedirects, bool bResetIndirectionsIfFailed) const
{
	auto GetAuthoritativeStruct = [](const UStruct* InStruct)->const UStruct*
		{
			if (const UClass* Class = Cast<UClass>(InStruct))
			{
				return Class->GetAuthoritativeClass();
			}

			return InStruct;
		};

	OutIndirections.Reset();
	if (OutError)
	{
		OutError->Reset();
	}
	
	// Nothing to do for an empty path.
	if (IsPathEmpty())
	{
		return true;
	}

	const uint8* CurrentAddress = static_cast<const uint8*>(BaseValueView.GetMemory());
	const UStruct* CurrentStruct = BaseValueView.GetStruct();
	
	for (const TEnumerateRef<const FPropertyBindingPathSegment> Segment : EnumerateRange(Segments))
	{
		if (CurrentStruct == nullptr)
		{
			if (OutError)
			{
				*OutError = FString::Printf(TEXT("Malformed path '%s'."),
					*ToString(Segment.GetIndex(), TEXT("<"), TEXT(">")));
			}

			if (bResetIndirectionsIfFailed)
			{
				OutIndirections.Reset();
			}
			else
			{
				// A binding path Foo.Bar.X becomes invalid when variable X doesn't exist anymore. Add a dummy indirection to indicate the segment index of X.
				FPropertyBindingPathIndirection& DummyIndirection = OutIndirections.Emplace_GetRef();
				DummyIndirection.PathSegmentIndex = Segment.GetIndex();
			}

			return false;
		}

		FName PropertyName = Segment->GetName();
		const FProperty* Property = CurrentStruct->FindPropertyByName(PropertyName);
		const bool bWithValue = CurrentAddress != nullptr;

#if WITH_EDITORONLY_DATA
		FName DisplayName = Segment->GetDisplayName();
		FName RedirectedName;
		FGuid PropertyGuid = Segment->GetPropertyGuid();

		// Try to fix the path in editor.
		if (bHandleRedirects)
		{
			if (!Property && PropertyGuid.IsValid())
			{
				// Try to match by property ID (Blueprint, User Defined Struct, PropertyBag).
				if (CurrentStruct->ArePropertyGuidsAvailable())
				{
					const FName NewName = CurrentStruct->FindPropertyNameFromGuid(PropertyGuid);
					if (!NewName.IsNone() && NewName != PropertyName)
					{
						RedirectedName = NewName;
						PropertyName = NewName;
						Property = CurrentStruct->FindPropertyByName(PropertyName);
					}
				}
			}

			// Check if there's a core redirect for it.
			if (!Property)
			{
				// Try core redirect
				for (const UStruct* CheckStruct : CurrentStruct->GetSuperStructIterator())
				{
					const FName NewPropertyName = FProperty::FindRedirectedPropertyName(CheckStruct, PropertyName);
					if (!NewPropertyName.IsNone())
					{
						// Cached the result for later use.
						RedirectedName = NewPropertyName;
						PropertyName = RedirectedName;
						Property = CurrentStruct->FindPropertyByName(PropertyName);
						break;
					}
				}
			}
		}
#endif // WITH_EDITORONLY_DATA

		// Check if it has a custom name
		if (!Property)
		{
			Property = CurrentStruct->CustomFindProperty(PropertyName);

#if WITH_EDITORONLY_DATA
			if(!Property && !DisplayName.IsNone())
			{
				Property = CurrentStruct->CustomFindProperty(DisplayName);
			}
#endif // WITH_EDITORONLY_DATA

			if (Property)
			{
				PropertyName = Property->GetFName();
			}
		}

		if (!Property)
		{
			if (OutError)
			{
				*OutError = FString::Printf(TEXT("Malformed path '%s', could not find property '%s%s::%s'."),
					*ToString(Segment.GetIndex(), TEXT("<"), TEXT(">")),
					CurrentStruct->GetPrefixCPP(), *CurrentStruct->GetName(), *Segment->GetName().ToString());
			}

			if (bResetIndirectionsIfFailed)
			{
				OutIndirections.Reset();
			}
			else
			{
				// A binding path Foo.Bar.X becomes invalid when variable X doesn't exist anymore. Add a dummy indirection to indicate the container struct Bar even X is invalid.
				FPropertyBindingPathIndirection& DummyIndirection = OutIndirections.Emplace_GetRef();
				DummyIndirection.ContainerStruct = CurrentStruct;
				DummyIndirection.ContainerAddress = CurrentAddress;
				DummyIndirection.PathSegmentIndex = Segment.GetIndex();
			}

			return false;
		}

#if WITH_EDITORONLY_DATA
		// Update PropertyGuid
		if (bHandleRedirects && CurrentStruct->ArePropertyGuidsAvailable())
		{
			PropertyGuid = CurrentStruct->FindPropertyGuidFromName(PropertyName);
		}
		DisplayName = FName(Property->GetDisplayNameText().ToString());
#endif // WITH_EDITORONLY_DATA

		const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
		int ArrayIndex = INDEX_NONE;
		int32 Offset = INDEX_NONE;

		checkf(Segment->GetArrayIndex() >= INDEX_NONE, TEXT("Segment Array Index is malformed."));

		if (ArrayProperty && Segment->GetArrayIndex() != INDEX_NONE)
		{
			FPropertyBindingPathIndirection& Indirection = OutIndirections.AddDefaulted_GetRef();
			Indirection.Property = Property;
			Indirection.ContainerAddress = CurrentAddress;
			Indirection.ContainerStruct = CurrentStruct;
			Indirection.InstanceStruct = nullptr;
			Indirection.ArrayIndex = Segment->GetArrayIndex();
			Indirection.PropertyOffset = ArrayProperty->GetOffset_ForInternal();
			Indirection.PathSegmentIndex = Segment.GetIndex();
			Indirection.AccessType = EPropertyBindingPropertyAccessType::IndexArray;
#if WITH_EDITORONLY_DATA
			Indirection.RedirectedName = RedirectedName;
			Indirection.PropertyGuid = PropertyGuid;
			Indirection.DisplayName = DisplayName;
#endif

			ArrayIndex = INDEX_NONE;
			Offset = 0;
			Property = ArrayProperty->Inner;

			if (bWithValue)
			{
				FScriptArrayHelper Helper(ArrayProperty, CurrentAddress + ArrayProperty->GetOffset_ForInternal());
				if (!Helper.IsValidIndex(Segment->GetArrayIndex()))
				{
					if (OutError)
					{
						*OutError = FString::Printf(TEXT("Index %d out of range (num elements %d) trying to access dynamic array '%s'."),
							Segment->GetArrayIndex(), Helper.Num(), *ToString(Segment.GetIndex(), TEXT("<"), TEXT(">")));
					}

					if (bResetIndirectionsIfFailed)
					{
						OutIndirections.Reset();
					}

					return false;
				}
				CurrentAddress = Helper.GetRawPtr(Segment->GetArrayIndex());
			}
		}
		else
		{
			const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
			if (StructProperty && StructProperty->Struct == TBaseStructure<FInstancedStructContainer>::Get())
			{
				Offset = Property->GetOffset_ForInternal();
			}
			else
			{
				if (Segment->GetArrayIndex() >= Property->ArrayDim)
				{
					if (OutError)
					{
						*OutError = FString::Printf(TEXT("Index %d out of range %d trying to access static array '%s'."),
							Segment->GetArrayIndex(), Property->ArrayDim, *ToString(Segment.GetIndex(), TEXT("<"), TEXT(">")));
					}

					if (bResetIndirectionsIfFailed)
					{
						OutIndirections.Reset();
					}

					return false;
				}
				Offset = Property->GetOffset_ForInternal() + Property->GetElementSize() * FMath::Max(0, Segment->GetArrayIndex());
			}

			ArrayIndex = Segment->GetArrayIndex();
		}

		FPropertyBindingPathIndirection& Indirection = OutIndirections.AddDefaulted_GetRef();
		Indirection.Property = Property;
		Indirection.ContainerAddress = CurrentAddress;
		Indirection.ContainerStruct = CurrentStruct;
		Indirection.ArrayIndex = ArrayIndex;
		Indirection.PropertyOffset = Offset;
		Indirection.PathSegmentIndex = Segment.GetIndex();
		Indirection.AccessType = EPropertyBindingPropertyAccessType::Offset; 
#if WITH_EDITORONLY_DATA
		Indirection.RedirectedName = RedirectedName;
		Indirection.PropertyGuid = PropertyGuid;
		Indirection.DisplayName = DisplayName;
#endif
		const bool bLastSegment = Segment.GetIndex() == (Segments.Num() - 1);

		if (!bLastSegment)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (bWithValue)
				{
					// For StructInstance types:
					// The property path is pointing into the instanced struct, it must be present.
					// @TODO:	We could potentially check the BaseStruct metadata in editor (for similar behavior as objects)
					//			Omitting for now to have matching functionality in editor and runtime.
					auto SetStructInstanceIndirection = [&GetAuthoritativeStruct, this, bResetIndirectionsIfFailed, &CurrentAddress, &CurrentStruct, &Indirection, &OutError, &OutIndirections]
						(const UScriptStruct* ScriptStruct, const uint8* StructMemory, const int32 SegmentIndex, const UStruct* SegmentScriptStruct)
						{
							if (ScriptStruct == nullptr || StructMemory == nullptr)
							{
								if (OutError)
								{
									*OutError = FString::Printf(TEXT("Expecting valid instanced struct value at path '%s'."),
										*ToString(SegmentIndex, TEXT("<"), TEXT(">")));
								}

								if (bResetIndirectionsIfFailed)
								{
									OutIndirections.Reset();
								}

								return false;
							}

							CurrentAddress = StructMemory;
							CurrentStruct = GetAuthoritativeStruct(ScriptStruct);
							Indirection.InstanceStruct = CurrentStruct;
							Indirection.ArrayIndex = INDEX_NONE;
							return true;
						};

					if (StructProperty->Struct == TBaseStructure<FInstancedStruct>::Get())
					{
						const FInstancedStruct& InstancedStruct = *reinterpret_cast<const FInstancedStruct*>((uint8*)CurrentAddress + Offset);
						if (!SetStructInstanceIndirection(InstancedStruct.GetScriptStruct(), InstancedStruct.GetMemory(), Segment.GetIndex(), Segment->GetInstanceStruct()))
						{
							return false;
						}
						Indirection.AccessType = EPropertyBindingPropertyAccessType::StructInstance;
					}
					else if (StructProperty->Struct == TBaseStructure<FSharedStruct>::Get())
					{
						const FSharedStruct& SharedStruct = *reinterpret_cast<const FSharedStruct*>((uint8*)CurrentAddress + Offset);
						if (!SetStructInstanceIndirection(SharedStruct.GetScriptStruct(), SharedStruct.GetMemory(), Segment.GetIndex(), Segment->GetInstanceStruct()))
						{
							return false;
						}
						Indirection.AccessType = EPropertyBindingPropertyAccessType::SharedStruct;
					}
					else if (StructProperty->Struct == TBaseStructure<FInstancedStructContainer>::Get())
					{
						const FInstancedStructContainer& InstancedStructContainer = *reinterpret_cast<const FInstancedStructContainer*>((uint8*)CurrentAddress + Offset);
						check(InstancedStructContainer.Num() > ArrayIndex);

						const FConstStructView StructView = InstancedStructContainer[ArrayIndex];
						if (!SetStructInstanceIndirection(StructView.GetScriptStruct(), StructView.GetMemory(), Segment.GetIndex(), Segment->GetInstanceStruct()))
						{
							return false;
						}
						Indirection.AccessType = EPropertyBindingPropertyAccessType::StructInstanceContainer;
						Indirection.ArrayIndex = ArrayIndex;
					}
					else
					{
						CurrentAddress = (uint8*)CurrentAddress + Offset;
						CurrentStruct = StructProperty->Struct;
						Indirection.AccessType = EPropertyBindingPropertyAccessType::Offset;
					}
				}
				else
				{
					if (Segment->GetInstanceStruct())
					{
						CurrentStruct = GetAuthoritativeStruct(Segment->GetInstanceStruct());
						Indirection.InstanceStruct = CurrentStruct;
						Indirection.AccessType = Segment->GetInstancedStructAccessType();
					}
					else
					{
						CurrentStruct = GetAuthoritativeStruct(StructProperty->Struct);
						Indirection.AccessType = EPropertyBindingPropertyAccessType::Offset;
					}
				}
			}
			else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				if (bWithValue)
				{
					const UObject* Object = *reinterpret_cast<UObject* const*>(CurrentAddress + Offset);
					CurrentAddress = reinterpret_cast<const uint8*>(Object);
					
					// The property path is pointing into the object, if the object is present use it's specific type, otherwise use the type of the pointer.
					if (Object)
					{
						CurrentStruct = GetAuthoritativeStruct(Object->GetClass());
						Indirection.InstanceStruct = CurrentStruct;
						Indirection.AccessType = EPropertyBindingPropertyAccessType::ObjectInstance;
					}
					else
					{
						CurrentStruct = GetAuthoritativeStruct(ObjectProperty->PropertyClass);
						Indirection.AccessType = EPropertyBindingPropertyAccessType::Object;
					}
				}
				else
				{
					if (Segment->GetInstanceStruct())
					{
						CurrentStruct = GetAuthoritativeStruct(Segment->GetInstanceStruct());
						Indirection.InstanceStruct = CurrentStruct;
						Indirection.AccessType = EPropertyBindingPropertyAccessType::ObjectInstance;
					}
					else
					{
						CurrentStruct = GetAuthoritativeStruct(ObjectProperty->PropertyClass);
						Indirection.AccessType = EPropertyBindingPropertyAccessType::Object;
					}
				}
			}
			// Check to see if this is a simple weak object property (eg. not an array of weak objects).
			else if (const FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Property))
			{
				if (bWithValue)
				{
					const FWeakObjectPtr& WeakObjectPtr = *reinterpret_cast<const FWeakObjectPtr*>(CurrentAddress + Offset);
					const UObject* Object = WeakObjectPtr.Get();
					CurrentAddress = reinterpret_cast<const uint8*>(Object);

					if (Object)
					{
						CurrentStruct = GetAuthoritativeStruct(Object->GetClass());
						Indirection.InstanceStruct = CurrentStruct;
					}
				}
				else
				{
					CurrentStruct = GetAuthoritativeStruct(WeakObjectProperty->PropertyClass);
				}

				Indirection.AccessType = EPropertyBindingPropertyAccessType::WeakObject;
			}
			// Check to see if this is a simple soft object property (eg. not an array of soft objects).
			else if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
			{
				if (bWithValue)
				{
					const FSoftObjectPtr& SoftObjectPtr = *reinterpret_cast<const FSoftObjectPtr*>(CurrentAddress + Offset);
					const UObject* Object = SoftObjectPtr.Get();
					CurrentAddress = reinterpret_cast<const uint8*>(Object);

					if (Object)
					{
						CurrentStruct = GetAuthoritativeStruct(Object->GetClass());
						Indirection.InstanceStruct = CurrentStruct;
					}
				}
				else
				{
					CurrentStruct = GetAuthoritativeStruct(SoftObjectProperty->PropertyClass);
				}

				Indirection.AccessType = EPropertyBindingPropertyAccessType::SoftObject;
			}
			else
			{
				// We get here if we encounter a property type that is not supported for indirection (e.g. Map or Set).
				if (OutError)
				{
					*OutError = FString::Printf(TEXT("Unsupported property indirection type %s in path '%s'."),
						*Property->GetCPPType(), *ToString(Segment.GetIndex(), TEXT("<"), TEXT(">")));
				}

				if (bResetIndirectionsIfFailed)
				{
					OutIndirections.Reset();
				}

				return false;
			}
		}
	}

	return true;
}

bool FPropertyBindingPath::ResolveIndirectionsWithValue(const FPropertyBindingDataView BaseValueView, TArray<FPropertyBindingPathIndirection>& OutIndirections, FString* OutError, bool bHandleRedirects, bool bResetIndirectionsIfFailed) const
{
	return ResolveIndirectionsWithValue<>(BaseValueView, OutIndirections, OutError, bHandleRedirects, bResetIndirectionsIfFailed);
}

bool FPropertyBindingPath::ResolveIndirectionsWithValue(const FPropertyBindingDataView BaseValueView, TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>>& OutIndirections, FString* OutError, bool bHandleRedirects, bool bResetIndirectionsIfFailed) const
{
	return ResolveIndirectionsWithValue<>(BaseValueView, OutIndirections, OutError, bHandleRedirects, bResetIndirectionsIfFailed);
}

bool FPropertyBindingPath::operator==(const FPropertyBindingPath& RHS) const
{
#if WITH_EDITORONLY_DATA
	if (StructID != RHS.StructID)
	{
		return false;
	}
#endif // WITH_EDITORONLY_DATA
	if (Segments.Num() != RHS.Segments.Num())
	{
		return false;
	}

	for (TEnumerateRef<const FPropertyBindingPathSegment> Segment : EnumerateRange(Segments))
	{
		if (*Segment != RHS.Segments[Segment.GetIndex()])
		{
			return false;
		}
	}

	return true;
}