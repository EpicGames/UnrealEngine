// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBindingSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyBindingSettings)

UPropertyBindingSettings::UPropertyBindingSettings()
{
	//These metadata are needed to keep the display.
	//Additional metadata can be added to the ini file.
	MetaDataToKeepWhenPromotingToParameter.Add("Bitmask");
	MetaDataToKeepWhenPromotingToParameter.Add("BitmaskEnum");
}
