// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "PropertyBindingSettings.generated.h"

UCLASS(MinimalAPI, config = Engine, defaultconfig)
class UPropertyBindingSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPropertyBindingSettings();

	/**
	 * Meta-data that are not in the MetaDataToKeepWhenPromotingToParameter are removed when a property is promoted.
	 */
	UPROPERTY(EditAnywhere, config, Category = "Property Binding")
	TArray<FName> MetaDataToKeepWhenPromotingToParameter;
};