// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBindingTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyBindingTypes)

DEFINE_LOG_CATEGORY(LogPropertyBindingUtils);

namespace UE::PropertyBinding
{
#if WITH_EDITORONLY_DATA
	TArray<TFunction<bool(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, TNotNull<FPropertyBindingIndex16*> Index)>> PropertyBindingIndex16ConversionFuncList;
#endif
}

UE::PropertyBinding::EPropertyCompatibility UE::PropertyBinding::GetPropertyCompatibility(const FProperty* FromProperty, const FProperty* ToProperty)
{
	if (FromProperty == ToProperty)
	{
		return EPropertyCompatibility::Compatible;
	}

	if (FromProperty == nullptr || ToProperty == nullptr)
	{
		return EPropertyCompatibility::Incompatible;
	}

	// Special case for object properties since InPropertyA->SameType(InPropertyB) requires both properties to be of the exact same class.
	// In our case we want to be able to bind a source property if its class is a child of the target property class.
	if (FromProperty->IsA<FObjectPropertyBase>() && ToProperty->IsA<FObjectPropertyBase>())
	{
		const FObjectPropertyBase* SourceProperty = CastField<FObjectPropertyBase>(FromProperty);
		const FObjectPropertyBase* TargetProperty = CastField<FObjectPropertyBase>(ToProperty);
		return (SourceProperty->PropertyClass->IsChildOf(TargetProperty->PropertyClass)) ? EPropertyCompatibility::Compatible : EPropertyCompatibility::Incompatible;
	}

	// When copying to an enum property, expect FromProperty to be the same enum.
	auto GetPropertyEnum = [](const FProperty* Property) -> const UEnum*
	{
		if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			return ByteProperty->GetIntPropertyEnum();
		}
		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			return EnumProperty->GetEnum();
		}
		return nullptr;
	};
	
	if (const UEnum* ToPropertyEnum = GetPropertyEnum(ToProperty))
	{
		const UEnum* FromPropertyEnum = GetPropertyEnum(FromProperty);
		return (ToPropertyEnum == FromPropertyEnum) ? EPropertyCompatibility::Compatible : EPropertyCompatibility::Incompatible;
	}
	
	// Allow source enums to be promoted to numbers.
	if (const FEnumProperty* EnumPropertyA = CastField<const FEnumProperty>(FromProperty))
	{
		FromProperty = EnumPropertyA->GetUnderlyingProperty();
	}

	if (FromProperty->SameType(ToProperty))
	{
		return EPropertyCompatibility::Compatible;
	}
	else
	{
		// Not directly compatible, check for promotions
		if (FromProperty->IsA<FBoolProperty>())
		{
			if (ToProperty->IsA<FByteProperty>()
				|| ToProperty->IsA<FIntProperty>()
				|| ToProperty->IsA<FUInt32Property>()
				|| ToProperty->IsA<FInt64Property>()
				|| ToProperty->IsA<FFloatProperty>()
				|| ToProperty->IsA<FDoubleProperty>())
			{
				return EPropertyCompatibility::Promotable;
			}
		}
		else if (FromProperty->IsA<FByteProperty>())
		{
			if (ToProperty->IsA<FIntProperty>()
				|| ToProperty->IsA<FUInt32Property>()
				|| ToProperty->IsA<FInt64Property>()
				|| ToProperty->IsA<FFloatProperty>()
				|| ToProperty->IsA<FDoubleProperty>())
			{
				return EPropertyCompatibility::Promotable;
			}
		}
		else if (FromProperty->IsA<FIntProperty>())
		{
			if (ToProperty->IsA<FInt64Property>()
				|| ToProperty->IsA<FFloatProperty>()
				|| ToProperty->IsA<FDoubleProperty>())
			{
				return EPropertyCompatibility::Promotable;
			}
		}
		else if (FromProperty->IsA<FUInt32Property>())
		{
			if (ToProperty->IsA<FInt64Property>()
				|| ToProperty->IsA<FFloatProperty>()
				|| ToProperty->IsA<FDoubleProperty>())
			{
				return EPropertyCompatibility::Promotable;
			}
		}
		else if (FromProperty->IsA<FFloatProperty>())
		{
			if (ToProperty->IsA<FIntProperty>()
				|| ToProperty->IsA<FInt64Property>()
				|| ToProperty->IsA<FDoubleProperty>())
			{
				return EPropertyCompatibility::Promotable;
			}
		}
		else if (FromProperty->IsA<FDoubleProperty>())
		{
			if (ToProperty->IsA<FIntProperty>()
				|| ToProperty->IsA<FInt64Property>()
				|| ToProperty->IsA<FFloatProperty>())
			{
				return EPropertyCompatibility::Promotable;
			}
		}
		else
		{
			const FArrayProperty* FromArrayProperty = CastField<const FArrayProperty>(FromProperty);
			const FArrayProperty* ToArrayProperty = CastField<const FArrayProperty>(ToProperty);
			if (FromArrayProperty && ToArrayProperty)
			{
				if (UE::PropertyBinding::GetPropertyCompatibility(FromArrayProperty->Inner, ToArrayProperty->Inner) != EPropertyCompatibility::Incompatible)
				{
					return EPropertyCompatibility::Promotable;
				}
			}
		}
	}

	return EPropertyCompatibility::Incompatible;
}

void UE::PropertyBinding::CreateUniquelyNamedPropertiesInPropertyBag(const TArrayView<FPropertyCreationDescriptor> InOutCreationDescs, FInstancedPropertyBag& OutPropertyBag)
{
	TArray<FPropertyBagPropertyDesc, TInlineAllocator<1>> PropertyDescs;
	PropertyDescs.Reserve(InOutCreationDescs.Num());

	// Generate unique names for the incoming property descs to avoid changing the existing properties in the bag
	for (FPropertyCreationDescriptor& CreationDesc : InOutCreationDescs)
	{
		int32 Index = CreationDesc.PropertyDesc.Name.GetNumber();
		while (OutPropertyBag.FindPropertyDescByName(CreationDesc.PropertyDesc.Name))
		{
			CreationDesc.PropertyDesc.Name = FName(CreationDesc.PropertyDesc.Name, Index++);
		}
		PropertyDescs.Add(CreationDesc.PropertyDesc);
	}

	OutPropertyBag.AddProperties(PropertyDescs);

	for (const FPropertyCreationDescriptor& CreationDesc : InOutCreationDescs)
	{
		// Attempt to copy the value from the Source Property / Address to Property Desc
		// There could be Type Mismatches if the Descs don't match the Source Property, but attempt to do it on all property descs
		if (CreationDesc.SourceProperty && CreationDesc.SourceContainerAddress)
		{
			OutPropertyBag.SetValue(CreationDesc.PropertyDesc.Name, CreationDesc.SourceProperty, CreationDesc.SourceContainerAddress);
		}
	}
}

bool FPropertyBindingIndex16::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_UInt16Property)
	{
		// Support loading from uint16.
		// Note: 0xffff is silently read as invalid value.
		uint16 OldValue = 0;
		Slot << OldValue;

		*this = FPropertyBindingIndex16(OldValue);
		return true;
	}
#if WITH_EDITORONLY_DATA
	if (Tag.Type == NAME_StructProperty)
	{
		for (auto& Func : UE::PropertyBinding::PropertyBindingIndex16ConversionFuncList)
		{
			if (Func(Tag, Slot, this))
			{
				return true;
			}
		}

		return true;
	}
#endif // WITH_EDITORONLY_DATA

	return false;
}
