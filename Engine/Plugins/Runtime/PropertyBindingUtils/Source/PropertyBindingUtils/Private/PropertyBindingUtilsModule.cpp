// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBindingUtilsModule.h"

#define LOCTEXT_NAMESPACE "PropertyBindingUtils"

class FPropertyBindingUtilsModule : public IPropertyBindingUtilsModule
{
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FPropertyBindingUtilsModule, PropertyBindingUtils)

void FPropertyBindingUtilsModule::StartupModule()
{
}

void FPropertyBindingUtilsModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
