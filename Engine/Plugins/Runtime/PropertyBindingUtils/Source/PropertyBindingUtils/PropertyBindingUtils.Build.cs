// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PropertyBindingUtils : ModuleRules
	{
		public PropertyBindingUtils(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
			}
			);

			PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DeveloperSettings",
				"RenderCore",
				"InputCore"
			}
			);

			if (Target.bBuildEditor)
			{
				PublicDependencyModuleNames.AddRange(
					new string[] {
						"UnrealEd",
						"BlueprintGraph",
					}
				);
			}
		}
	}
}
