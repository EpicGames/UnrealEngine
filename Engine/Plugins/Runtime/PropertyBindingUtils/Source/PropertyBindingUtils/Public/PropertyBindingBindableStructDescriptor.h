// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingBindableStructDescriptor.generated.h"

/**
 * Descriptor for a struct or class that can be a binding source or target.
 * Each struct has unique identifier, which is used to distinguish them, and name that is mostly for debugging and UI.
 */
USTRUCT()
struct FPropertyBindingBindableStructDescriptor
{
	GENERATED_BODY()

	FPropertyBindingBindableStructDescriptor() = default;
	PROPERTYBINDINGUTILS_API virtual ~FPropertyBindingBindableStructDescriptor();

#if WITH_EDITORONLY_DATA
	FPropertyBindingBindableStructDescriptor(const FName InName, const UStruct* InStruct, const FGuid InGuid)
		: Struct(InStruct)
		, Name(InName)
		, ID(InGuid)
	{
	}

	bool operator==(const FPropertyBindingBindableStructDescriptor& RHS) const
	{
		return ID == RHS.ID && Struct == RHS.Struct; // Not checking name, it's cosmetic.
	}

	/** Optional section that can be provided to the UI to organize the menus. */
	virtual FString GetSection() const
	{
		return {};
	}
#endif // WITH_EDITORONLY_DATA

	bool IsValid() const
	{
		return Struct != nullptr;
	}

	PROPERTYBINDINGUTILS_API virtual FString ToString() const;
	
	/** The type of the struct or class. */
	UPROPERTY()
	TObjectPtr<const UStruct> Struct = nullptr;

	/** Display name of the struct (used for debugging, logging, cosmetic), different from Struct->GetName(). */
	UPROPERTY()
	FName Name;

#if WITH_EDITORONLY_DATA
	/** Unique identifier of the struct. */
	UPROPERTY()
	FGuid ID;

	/** Category of the bindable struct. Can be used to display the category in a menu. */
	UPROPERTY()
	FString Category;
#endif
};
