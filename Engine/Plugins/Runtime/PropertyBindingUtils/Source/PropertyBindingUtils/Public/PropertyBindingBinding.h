// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingPath.h"
#include "PropertyBindingBinding.generated.h"

/**
 * Representation of a property binding
 */
USTRUCT()
struct FPropertyBindingBinding
{
	GENERATED_BODY()

	FPropertyBindingBinding() = default;
	
	FPropertyBindingBinding(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath)
		: SourcePropertyPath(InSourcePath)
		, TargetPropertyPath(InTargetPath)
	{
	}

	FPropertyBindingBinding(const FPropertyBindingBinding& Other)
		: SourcePropertyPath(Other.SourcePropertyPath)
		, TargetPropertyPath(Other.TargetPropertyPath)
#if WITH_EDITOR
		, PropertyFunctionNode(Other.PropertyFunctionNode)
#endif
	{
	}

	FPropertyBindingBinding& operator=(const FPropertyBindingBinding& Other)
	{
		if (this == &Other)
		{
			return *this;
		}
		SourcePropertyPath = Other.SourcePropertyPath;
		TargetPropertyPath = Other.TargetPropertyPath;
#if WITH_EDITOR
		PropertyFunctionNode = Other.PropertyFunctionNode;
#endif
		return *this;
	}

	FPropertyBindingBinding(FPropertyBindingBinding&& Other)
		: SourcePropertyPath(MoveTemp(Other.SourcePropertyPath))
		, TargetPropertyPath(MoveTemp(Other.TargetPropertyPath))
#if WITH_EDITOR
		, PropertyFunctionNode(MoveTemp(Other.PropertyFunctionNode))
#endif
	{
	}

	FPropertyBindingBinding& operator=(FPropertyBindingBinding&& Other)
	{
		if (this == &Other)
		{
			return *this;
		}
		SourcePropertyPath = MoveTemp(Other.SourcePropertyPath);
		TargetPropertyPath = MoveTemp(Other.TargetPropertyPath);
#if WITH_EDITOR
		PropertyFunctionNode = MoveTemp(Other.PropertyFunctionNode);
#endif
		return *this;
	}

#if WITH_EDITOR
	FPropertyBindingBinding(FConstStructView InFunctionNodeStruct, const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath)
		: SourcePropertyPath(InSourcePath)
		, TargetPropertyPath(InTargetPath)
		, PropertyFunctionNode(InFunctionNodeStruct)
	{
	}
#endif

	virtual ~FPropertyBindingBinding() = default;

	const FPropertyBindingPath& GetSourcePath() const
	{
		return SourcePropertyPath;
	}

	const FPropertyBindingPath& GetTargetPath() const
	{
		return TargetPropertyPath;
	}

	FPropertyBindingPath& GetMutableSourcePath()
	{
		return SourcePropertyPath;
	}

	FPropertyBindingPath& GetMutableTargetPath()
	{
		return TargetPropertyPath;
	}

	virtual FConstStructView GetSourceDataHandleStruct() const PURE_VIRTUAL(FPropertyBindingBinding::GetSourceDataHandle, return {};);

#if WITH_EDITOR
	FConstStructView GetPropertyFunctionNode() const
	{
		return FConstStructView(PropertyFunctionNode);
	}

	FStructView GetMutablePropertyFunctionNode()
	{
		return FStructView(PropertyFunctionNode);
	}
#endif // WITH_EDITOR

	PROPERTYBINDINGUTILS_API FString ToString() const;

protected:
	/** Source property path of the binding */
	UPROPERTY(VisibleAnywhere, Category = "Bindings")
	FPropertyBindingPath SourcePropertyPath;

	/** Target property path of the binding */
	UPROPERTY(VisibleAnywhere, Category = "Bindings")
	FPropertyBindingPath TargetPropertyPath;

#if WITH_EDITORONLY_DATA
	/**	Instance of bound PropertyFunction. */
	UPROPERTY(VisibleAnywhere, Category = "Bindings")
	FInstancedStruct PropertyFunctionNode;
#endif // WITH_EDITORONLY_DATA
};

template<>
struct TStructOpsTypeTraits<FPropertyBindingBinding> :
	TStructOpsTypeTraitsBase2<FPropertyBindingBinding>
{
	enum
	{
		WithPureVirtual = true,
	};
};

namespace UE::PropertyBinding
{
	extern PROPERTYBINDINGUTILS_API FString GetDescriptorAndPathAsString(const FPropertyBindingBindableStructDescriptor& InDescriptor, const FPropertyBindingPath& InPath);
}
