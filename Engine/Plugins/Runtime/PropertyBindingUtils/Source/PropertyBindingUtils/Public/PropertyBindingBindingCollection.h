// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingPath.h"
#include "PropertyBindingTypes.h"
#include "UObject/Interface.h"
#include "UObject/WeakInterfacePtr.h"
#include "PropertyBindingBindingCollection.generated.h"

struct FPropertyBindingBindableStructDescriptor;
struct FPropertyBindingBinding;
struct FPropertyBindingDataView;
struct FPropertyBindingPath;
class IPropertyBindingBindingCollectionOwner;

// @todo: Should have a common interface for binding iterator that supports removal op to remove the need of Internal virtual functions
/**
 * Base structure to inherit from to facilitate operations on property bindings.
 * @see FPropertyBindingExtension, IPropertyBindingBindingCollectionOwner
 */
USTRUCT()
struct FPropertyBindingBindingCollection
{
	GENERATED_BODY()

	virtual ~FPropertyBindingBindingCollection() = default;

	/** Result returned from a visitor functor indicating to continue or to quit early */
	enum class EVisitResult : uint8
	{
		/** Stop iterating through bindings and early exit */
		Break,
		/**  Continue to iterate through all bindings */
		Continue
	};

	/** Get the associated bindings owner. */
	IPropertyBindingBindingCollectionOwner* GetBindingsOwner() const
	{
		return BindingsOwner.GetInterface();
	}

	/** Get the associated bindings owner as a UObject. */
	UObject* GetBindingsOwnerObject() const
	{
		return BindingsOwner.GetObject();
	}

	/** Sets associated bindings owner, used to validate added property paths. */
	void SetBindingsOwner(TScriptInterface<IPropertyBindingBindingCollectionOwner> InBindingsOwner)
	{
		BindingsOwner = InBindingsOwner;
#if WITH_EDITOR
		OnBindingsOwnerSet(BindingsOwner);
#endif
	}

#if WITH_EDITOR
	/** Enum describing what is required for a binding path to be considered a match for a binding */
	enum class ESearchMode : uint8
	{
		/** Binding with exact matching path */
		Exact,
		/** Binding with path that matches but the binding path can be longer */
		Includes,
	};

	/** Can be overridden by derived classed to track newly assigned bindings owner interface. */
	virtual void OnBindingsOwnerSet(TScriptInterface<IPropertyBindingBindingCollectionOwner> InBindingsOwner)
	{
	}

	/**
	 * Adds binding between source and destination paths. Removes any bindings to InTargetPath before adding the new one.
	 * @param InSourcePath Binding source property path.
	 * @param InTargetPath Binding target property path.
	 */
	PROPERTYBINDINGUTILS_API void AddBinding(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath);

	/**
	 * Removes all bindings to target path.
	 * @param InTargetPath Target property path.
	 * @param InSearchMode Requirement for InTargetPath to be considered a match for a binding
	 */
	PROPERTYBINDINGUTILS_API void RemoveBindings(const FPropertyBindingPath& InTargetPath, ESearchMode InSearchMode = ESearchMode::Exact);

	/**
	 * Removes all bindings that match the predicate
	 * @param InPredicate Predicate to validate the binding
	 */
	PROPERTYBINDINGUTILS_API void RemoveBindings(TFunctionRef<bool(FPropertyBindingBinding&)> InPredicate);
	
	enum class EInvalidBindingCriteria : uint8
	{
		// Binding is removed if Binding Source Struct is invalid
		RemoveMissingSourceStruct = 1 << 0,
		// Binding is removed if Binding Target Struct is invalid
		RemoveMissingTargetStruct = 1 << 1,
		// Binding is removed if Binding Source Path failed to resolve
		RemoveFailedToResolveSourcePath = 1 << 2,
		// Binding is removed if Binding Target Path failed to resolve
		RemoveFailedToResolveTargetPath = 1 << 3,

		// Missing Source Struct is not included to be consistent with the old behavior 
		Default = RemoveMissingTargetStruct | RemoveFailedToResolveSourcePath | RemoveFailedToResolveTargetPath
	};

	/**
	 * Removes bindings which do not point to valid structs IDs.
	 * @param InValidStructs Set of struct IDs that are currently valid.
	 * @param InCriteria Rules for what bindings are regarded as invalid
	 */
	PROPERTYBINDINGUTILS_API void RemoveInvalidBindings(const TMap<FGuid, const FPropertyBindingDataView>& InValidStructs, EInvalidBindingCriteria InCriteria = EInvalidBindingCriteria::Default);

	/**
	 * Has any binding to the target path.
	 * @return True of the target path has any bindings.
	 */
	PROPERTYBINDINGUTILS_API bool HasBinding(const FPropertyBindingPath& InTargetPath, ESearchMode InSearchMode = ESearchMode::Exact) const;

	/**
	 * @return binding to the target path.
	 */
	PROPERTYBINDINGUTILS_API const FPropertyBindingBinding* FindBinding(const FPropertyBindingPath& InTargetPath, ESearchMode InSearchMode = ESearchMode::Exact) const;

	/**
	 * Copies property bindings from an existing struct to another.
	 * Overrides a binding to a specific property if it already exists in InToStructID.
	 * @param InFromStructID ID of the struct to copy from.
	 * @param InToStructID ID of the struct to copy to.
	 */
	PROPERTYBINDINGUTILS_API void CopyBindings(const FGuid InFromStructID, const FGuid InToStructID);
	
	/**
	 * @return Source path for given target path, or null if binding does not exist.
	 */
	PROPERTYBINDINGUTILS_API const FPropertyBindingPath* GetBindingSource(const FPropertyBindingPath& InTargetPath) const;
	
	/**
	 * Returns all pointers to bindings for a specified structs based in struct ID.
	 * @param InStructID ID of the struct to find bindings for.
	 * @param OutBindings Bindings for specified struct.
	 */
	PROPERTYBINDINGUTILS_API void GetBindingsFor(const FGuid InStructID, TArray<const FPropertyBindingBinding*>& OutBindings) const;

protected:
	virtual FPropertyBindingBinding* AddBindingInternal(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath) PURE_VIRTUAL(FPropertyBindingBindingCollection::AddBindingInternal, return nullptr; );
	PROPERTYBINDINGUTILS_API virtual void CopyBindingsInternal(const FGuid InFromStructID, const FGuid InToStructID);
	virtual void RemoveBindingsInternal(TFunctionRef<bool(FPropertyBindingBinding&)> InPredicate) PURE_VIRTUAL(FPropertyBindingBindingCollection::RemoveBindingsInternal, );
	virtual bool HasBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const PURE_VIRTUAL(FPropertyBindingBindingCollection::HasBindingInternal, return false; );
	virtual const FPropertyBindingBinding* FindBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const PURE_VIRTUAL(FPropertyBindingBindingCollection::FindBindingInternal, return nullptr; );

	/** Copies property bindings from an existing struct to another. */
	PROPERTYBINDINGUTILS_API void CopyBindingsImplementation(const FGuid InFromStructID, const FGuid InToStructID, TFunctionRef<bool(const FPropertyBindingBinding& Binding)> CanCopy);

	/** Update newly added binding's segments from struct value so that the segments are up to date. */
	PROPERTYBINDINGUTILS_API void UpdateSegmentsForNewlyAddedBinding(FPropertyBindingBinding& AddedBinding);
#endif // WITH_EDITOR
	const UObject* GetLogOwner() const;

public:
	FPropertyBindingCopyInfoBatch& AddCopyBatch()
	{
		return CopyBatches.AddDefaulted_GetRef();
	}

	int32 GetNumCopyBatches() const
	{
		return CopyBatches.Num();
	}
	
	const TArray<FPropertyBindingCopyInfoBatch>& GetCopyBatches() const
	{
		return CopyBatches;
	}

	TArray<FPropertyBindingCopyInfoBatch>& GetMutableCopyBatches()
	{
		return CopyBatches;
	}

	/** @return Number of bindable struct descriptors available in the collection. */
	virtual int32 GetNumBindableStructDescriptors() const PURE_VIRTUAL(FPropertyBindingBindingCollection::GetNumBindableStructDescriptors, return INDEX_NONE;);

	virtual const FPropertyBindingBindableStructDescriptor* GetBindableStructDescriptorFromHandle(FConstStructView InSourceHandleView) const PURE_VIRTUAL(FPropertyBindingBindingCollection::GetBindableStructDescriptorFromHandle, return nullptr; );

	/** @return The amount of registered bindings. */
	virtual int32 GetNumBindings() const PURE_VIRTUAL(FPropertyBindingBindingCollection::GetNumBindings, return INDEX_NONE;);

	/**
	 * Iterates through all bindings and calls the provided function on each non-mutable binding.
	 * @param InFunction Function to call on each binding.
	 * @note To be able to modify the binding, use ForEachMutableBinding
	 * @note To be able to break iteration, use VisitBindings or VisitMutableBindings
	 * @see ForEachMutableBinding, VisitBindings, VisitMutableBindings
	 */
	virtual void ForEachBinding(TFunctionRef<void(const FPropertyBindingBinding& Binding)> InFunction) const PURE_VIRTUAL(FPropertyBindingBindingCollection::ForEachBinding, );

	/**
	 * Iterates through all bindings between indices [Begin, End) and calls the provided function on each non-mutable binding.
	 * @param InBegin Index of the first binding to process
	 * @param InEnd Index to one past the last binding to process
	 * @param InFunction Function to call on each binding.
	 * @note To be able to modify the binding, use ForEachMutableBinding
	 * @note To be able to break iteration, use VisitBindings or VisitMutableBindings
	 * @see ForEachMutableBinding, VisitBindings, VisitMutableBindings
	 */
	virtual void ForEachBinding(FPropertyBindingIndex16 InBegin, FPropertyBindingIndex16 InEnd, TFunctionRef<void(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const PURE_VIRTUAL(FPropertyBindingBindingCollection::ForEachBinding, );

	/**
	 * Iterates through all bindings and calls the provided function on each mutable binding.
	 * @param InFunction Function to call on each binding.
	 * @note For read access only, consider using ForEachBinding
	 * @note To be able to break iteration, use VisitBindings or VisitMutableBindings
	 * @see ForEachBinding, VisitBindings, VisitMutableBindings
	 */
	virtual void ForEachMutableBinding(TFunctionRef<void(FPropertyBindingBinding& Binding)> InFunction) PURE_VIRTUAL(FPropertyBindingBindingCollection::ForEachMutableBinding, );

	/**
	 * Iterates through all bindings and calls the provided function on each non-mutable binding.
	 * That function must return whether the iteration should stop or not (i.e., EVisitResult::Break / Continue).
	 * @param InFunction Function to call on each binding and that indicates whether the visit should stop
	 * @note To be able to modify the binding use VisitMutableBindings
	 * @note If breaking the iteration is not required, consider using ForEachBinding or ForEachMutableBinding
	 * @see VisitMutableBindings, ForEachBinding, ForEachMutableBinding
	 */
	virtual void VisitBindings(TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding)> InFunction) const PURE_VIRTUAL(FPropertyBindingBindingCollection::VisitBindings, );
	
	/**
	 * Iterates all bindings between indices [Begin, End) and calls the provided function on each non-mutable binding.
	 * That function must return whether the iteration should stop or not (i.e., EVisitResult::Break / Continue).
	 * @param InBegin Index of the first binding to process
	 * @param InEnd Index to one past the last binding to process
	 * @param InFunction Function to call on each binding and that indicates whether the visit should stop
	 * @note To be able to modify the binding use VisitMutableBindings
	 * @note If breaking the iteration is not required, consider using ForEachBinding or ForEachMutableBinding
	 * @see VisitMutableBindings, ForEachBinding, ForEachMutableBinding
	 */
	virtual void VisitBindings(FPropertyBindingIndex16 InBegin, FPropertyBindingIndex16 InEnd, TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) 
		const PURE_VIRTUAL(FPropertyBindingBindingCollection::VisitBindings, );
	
	/**
	 * Iterates through all bindings and calls the provided function on each mutable binding.
	 * That function must return whether the iteration should stop or not (i.e., EVisitResult::Break / Continue).
	 * @param InFunction Function to call on each binding and that indicates whether the visit should stop.
	 * @note For read access only, consider using VisitBindings
	 * @note If breaking the iteration is not required, consider using ForEachBinding or ForEachMutableBinding
	 * @see VisitBindings, ForEachBinding, ForEachMutableBinding
	 */
	virtual void VisitMutableBindings(TFunctionRef<EVisitResult(FPropertyBindingBinding& Binding)> InFunction) PURE_VIRTUAL(FPropertyBindingBindingCollection::VisitMutableBindings, );
	
	/**
	 * Clears all bindings and related data.
	 */
	PROPERTYBINDINGUTILS_API void Reset();

	/**
	 * Optional virtual that derived classes could override to when bindings should be reset.
	 */
	virtual void OnReset()
	{
	}

	/**
	 * Resolves paths to indirections.
	 * @return True if resolve succeeded.
	 */
	[[nodiscard]] PROPERTYBINDINGUTILS_API bool ResolvePaths();

	/**
	 * Optional virtual that derived classes could override to resolve additional paths.
	 * @return True if resolve succeeded.
	 */
	[[nodiscard]] virtual bool OnResolvingPaths()
	{
		return true;
	}

	/**
	 * @return True if bindings have been resolved.
	 */
	bool IsValid() const
	{
		return bBindingsResolved;
	}

	/**
	 * Copies a property from Source to Target based on the provided Copy.
	 * @param Copy Describes which parameter and how is copied.
	 * @param SourceStructView Pointer and type for the source containing the property to be copied.
	 * @param TargetStructView Pointer and type for the target containing the property to be copied.
	 * @return true if the property was copied successfully.
	 */
	PROPERTYBINDINGUTILS_API bool CopyProperty(const FPropertyBindingCopyInfo& Copy, FPropertyBindingDataView SourceStructView, FPropertyBindingDataView TargetStructView) const;

	/** @return copy batch at specified index. */
	const FPropertyBindingCopyInfoBatch& GetBatch(const FPropertyBindingIndex16 TargetBatchIndex) const
	{
		check(TargetBatchIndex.IsValid());
		return CopyBatches[TargetBatchIndex.Get()];
	}

	/** @return All the property copies for a specific batch. */
	TConstArrayView<FPropertyBindingCopyInfo> GetBatchCopies(const FPropertyBindingIndex16 TargetBatchIndex) const
	{
		return GetBatchCopies(GetBatch(TargetBatchIndex));
	}

	/** @return All the property copies for a specific batch. */
	TConstArrayView<FPropertyBindingCopyInfo> GetBatchCopies(const FPropertyBindingCopyInfoBatch& Batch) const
	{
		const int32 Count = Batch.BindingsEnd.Get() - Batch.BindingsBegin.Get();
		if (Count == 0)
		{
			return {};
		}
		return MakeArrayView(&PropertyCopies[Batch.BindingsBegin.Get()], Count);
	}

	/**
	 * Resets copied properties in TargetStructView. Can be used e.g. to erase UObject references.
	 * @param TargetBatchIndex Batch index to copy (see FStateTreePropertyBindingCompiler).
	 * @param TargetStructView View to struct where properties are copied to.
	 * @return true if all resets succeeded (a reset can fail e.g. if source or destination struct view is invalid).
	 */
	PROPERTYBINDINGUTILS_API bool ResetObjects(const FPropertyBindingIndex16 TargetBatchIndex, FPropertyBindingDataView TargetStructView) const;

	/**
	 * @return true if any of the elements in the property bindings contains any of the structs in the set.
	 */
	PROPERTYBINDINGUTILS_API bool ContainsAnyStruct(const TSet<const UStruct*>& InStructs) const;

	/**
	 * Resolves copy info for a resolved binding.
	 * Will resolve CopyType, CopySize and Leaf Properties. The other information has already been set in ResolvePaths.
	 * @param InResolvedBinding Resolved Binding corresponding to the resulting copy type,
	 * @param InBindingSourceLeafIndirection Property path indirections of the leaf property of binding source,
	 * @param InBindingTargetLeafIndirection Property path indirections of the leaf property of binding target.
	 * @param OutCopyInfo Resulting copy type.
	 * @return true if copy was resolved, or false if no copy could be resolved.
	 */
	[[nodiscard]] PROPERTYBINDINGUTILS_API virtual bool ResolveBindingCopyInfo(const FPropertyBindingBinding& InResolvedBinding, const FPropertyBindingPathIndirection& InBindingSourceLeafIndirection, const FPropertyBindingPathIndirection& InBindingTargetLeafIndirection, FPropertyBindingCopyInfo& OutCopyInfo);

	UE_DEPRECATED(5.7, "ResolveCopyType is deprecated. Use ResolveBindingCopyInfo instead.")
	[[nodiscard]] static bool ResolveCopyType(const FPropertyBindingPathIndirection& SourceIndirection, const FPropertyBindingPathIndirection& TargetIndirection, FPropertyBindingCopyInfo& OutCopy, const UScriptStruct* StructReferenceType = nullptr)
	{
		return false;
	}

protected:
#if WITH_EDITOR || WITH_PROPERTYBINDINGUTILS_DEBUG
	[[nodiscard]] PROPERTYBINDINGUTILS_API FString DebugAsString() const;
#endif

	[[nodiscard]] PROPERTYBINDINGUTILS_API bool ResolvePath(const UStruct* Struct, const FPropertyBindingPath& Path, FPropertyBindingPropertyIndirection& OutFirstIndirection, FPropertyBindingPathIndirection& OutLeafIndirection);
	[[nodiscard]] PROPERTYBINDINGUTILS_API bool ResolvePath(const FPropertyBindingDataView DataView, const FPropertyBindingPath& Path, FPropertyBindingPropertyIndirection& OutFirstIndirection, FPropertyBindingPathIndirection& OutLeafIndirection);

	/**
	 * Resolves what kind of copy type to use between specified property indirections and what should be the correct copy size.
	 * @param InSourceIndirection Leaf path indirection of the copy source,
	 * @param InTargetIndirection Leaf path indirection of the copy target.
	 * @param OutCopyInfo copy info to resolve
	 * @return true if it was resolved, or false if it cannot be resolved between paths.
	 */
	[[nodiscard]] PROPERTYBINDINGUTILS_API bool ResolveCopyInfoBetweenIndirections(const FPropertyBindingPathIndirection& InSourceIndirection, const FPropertyBindingPathIndirection& InTargetIndirection, FPropertyBindingCopyInfo& OutCopyInfo) const;

	PROPERTYBINDINGUTILS_API bool PerformCopy(const FPropertyBindingCopyInfo& Copy, uint8* SourceAddress, uint8* TargetAddress) const;
	PROPERTYBINDINGUTILS_API void PerformResetObjects(const FPropertyBindingCopyInfo& Copy, uint8* TargetAddress) const;
	PROPERTYBINDINGUTILS_API uint8* GetAddress(FPropertyBindingDataView InStructView, const FPropertyBindingPropertyIndirection& FirstIndirection, const FProperty* LeafProperty) const;

	virtual void VisitSourceStructDescriptorInternal(TFunctionRef<EVisitResult(const FPropertyBindingBindableStructDescriptor& Descriptor)> InFunction) const
	{
	}

	//~ Following members are protected for deprecation reasons
protected:

	/** Array of copy batches. */
	UPROPERTY()
	TArray<FPropertyBindingCopyInfoBatch> CopyBatches;

	/** Array of property copies */
	UPROPERTY(Transient)
	TArray<FPropertyBindingCopyInfo> PropertyCopies;

private:

	/** Array of property indirections, indexed by accesses*/
	UPROPERTY(Transient)
	TArray<FPropertyBindingPropertyIndirection> PropertyIndirections;

	/** Flag indicating if the properties has been resolved successfully . */
	bool bBindingsResolved = false;

protected:
	UPROPERTY(Transient)
	TScriptInterface<IPropertyBindingBindingCollectionOwner> BindingsOwner = nullptr;

	/** Type for struct references copy type */
	TObjectPtr<UScriptStruct> PropertyReferenceStructType;

	/** Functor to handle property copy for EPropertyCopyType::StructReference */
	TFunction<void(const FStructProperty& SourceStructProperty, uint8* SourceAddress, uint8* TargetAddress)> PropertyReferenceCopyFunc;

	/** Functor to handle reset object for EPropertyCopyType::StructReference */
	TFunction<void(uint8* TargetAddress)> PropertyReferenceResetFunc;
};

template<>
struct TStructOpsTypeTraits<FPropertyBindingBindingCollection> : public TStructOpsTypeTraitsBase2<FPropertyBindingBindingCollection>
{
	enum
	{
		WithPureVirtual = true,
	};
};

#if WITH_EDITOR
ENUM_CLASS_FLAGS(FPropertyBindingBindingCollection::EInvalidBindingCriteria)
#endif //WITH_EDITOR