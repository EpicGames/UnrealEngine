// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"
#include "UObject/Interface.h"
#include "PropertyBindingBindingCollectionOwner.generated.h"

#define UE_API PROPERTYBINDINGUTILS_API

namespace UE::PropertyBinding
{
struct FPropertyCreationDescriptor;
}
struct FPropertyBindingBindableStructDescriptor;
struct FPropertyBindingBinding;
struct FPropertyBindingBindingCollection;
struct FPropertyBindingDataView;
struct FPropertyBindingPath;
template<typename BaseStructT> struct TInstancedStruct;

UINTERFACE(MinimalAPI)
class UPropertyBindingBindingCollectionOwner : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface that asset or their associated Editor-only data could inherit from to facilitate
 * operations on property bindings.
 * @see FPropertyBindingExtension
 */
class IPropertyBindingBindingCollectionOwner
{
	GENERATED_BODY()

public:

	enum class EBindingSide : uint8
	{
		Source,
		Target
	};

	/**
	 * Returns a data view for a given existing binding side.
	 * @param InBinding Binding to get a data view for
	 * @param InSide Side of the binding to get the data view for (source or target)
	 * @param OutDataView Result data view
	 * @return True if data view is found.
	 */
	virtual bool GetBindingDataView(const FPropertyBindingBinding& InBinding, EBindingSide InSide, FPropertyBindingDataView& OutDataView)
	{
		return false;
	}

#if WITH_EDITOR
	
	/**
	 * Returns structs within the owner that are visible for target struct.
	 * @param InTargetStructID Target struct ID
	 * @param OutStructDescs Result descriptors of the visible structs.
	 */
	virtual void GetBindableStructs(const FGuid InTargetStructID, TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& OutStructDescs) const = 0;

	/**
	 * Returns struct descriptor based on struct ID.
	 * @param InStructID Target struct ID
	 * @param OutStructDesc Result descriptor.
	 * @return True if struct found.
	 */
	virtual bool GetBindableStructByID(const FGuid InStructID, TInstancedStruct<FPropertyBindingBindableStructDescriptor>& OutStructDesc) const = 0;
		
	/**
	 * Returns data view based on struct ID.
	 * @param InStructID Target struct ID
	 * @param OutDataView Result data view.
	 * @return True if struct found.
	 */
	virtual bool GetBindingDataViewByID(const FGuid InStructID, FPropertyBindingDataView& OutDataView) const = 0;

	/** @return Pointer to editor property bindings. */
	virtual FPropertyBindingBindingCollection* GetEditorPropertyBindings() = 0;

	/** @return Pointer to const editor property bindings. */
	virtual const FPropertyBindingBindingCollection* GetEditorPropertyBindings() const = 0;
	
	/**
	 * Can be overridden to determine whether the struct matching the given struct id is capable of adding new properties.
	 * @param InStructID Target struct ID
	 * @return Whether the struct supports adding new properties
	 */
	virtual bool CanCreateParameter(const FGuid InStructID) const
	{
		return true;
	}

	/**
	 * Called when a property is being promoted to a parameter.
	 * @param PropertyDesc The newly created property descriptor
	 */
	UE_API virtual void OnPromotingToParameter(FPropertyBagPropertyDesc& PropertyDesc);

	/**
	 * Creates the given properties in the property bag of the struct matching the given struct ID
	 * @param InStructID Target struct ID
	 * @param InOutCreationDescs the descriptions of the properties to create. This is modified to update the property names that actually got created
	 */
	virtual void CreateParametersForStruct(const FGuid InStructID, TArrayView<UE::PropertyBinding::FPropertyCreationDescriptor> InOutCreationDescs) = 0;

	/** Can be overridden to provide a fallback structure ID to build the property path if no bindable structs are found when traversing a PropertyHandle hierarchy. */
	virtual FGuid GetFallbackStructID() const
	{
		// No fallback by default
		return {};
	}

	/** Can be overridden to provide additional bindable structs from property functions. */
	virtual void AppendBindablePropertyFunctionStructs(TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& InOutStructs) const
	{
	}

	/** Can be overridden to perform additional operations when property bindings changed. */
	virtual void OnPropertyBindingChanged(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath)
	{
	}
#endif // WITH_EDITOR
};

#undef UE_API