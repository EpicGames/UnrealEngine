// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructView.h"
#include "UObject/Class.h"
#include "UObject/Object.h"

/**
 * Short-lived pointer to an object or struct.
 * Can be constructed directly from a pointer to a UObject or from a StructView.
 *
 * When creating from a generic UStruct, the data view expects a valid type (UStruct) when a valid memory pointer is provided.
 * It that case it is also fine to provide null for both and the constructed data view will be considered as invalid.
 */
struct FPropertyBindingDataView
{
	FPropertyBindingDataView() = default;

	/**
	 * Generic struct constructor.
	 * Valid UStruct is required when passing in valid memory pointer.
	 * Both can be null and the constructed data view will be considered as invalid.
	 * @see IsValid
	 */
	FPropertyBindingDataView(const UStruct* InStruct, void* InMemory) : Struct(InStruct), Memory(InMemory)
	{
		// Must have type with valid pointer.
		check(!Memory || (Memory && Struct));
	}

	/** UObject constructor. */
	FPropertyBindingDataView(UObject* Object) : Struct(Object ? Object->GetClass() : nullptr), Memory(Object)
	{
		// Must have type with valid pointer.
		check(!Memory || (Memory && Struct));
	}

	/** Struct from a StructView. */
	FPropertyBindingDataView(const FStructView StructView) : Struct(StructView.GetScriptStruct()), Memory(StructView.GetMemory())
	{
		// Must have type with valid pointer.
		check(!Memory || (Memory && Struct));
	}

	/** Struct from a InstancedStruct. */
	FPropertyBindingDataView(FInstancedStruct& InstancedStruct) : Struct(InstancedStruct.GetScriptStruct()), Memory(InstancedStruct.GetMutableMemory())
	{
		// Must have type with valid pointer.
		check(!Memory || (Memory && Struct));
	}

	/**
	 * Check is the view is valid (both pointer and type are set). On valid views it is safe to call the Get<>() methods returning a reference.
	 * @return True if the view is valid.
	 */
	bool IsValid() const
	{
		return Memory != nullptr && Struct != nullptr;
	}

	/**
	 * UObject getters (reference & pointer, const & mutable)
	 */
	template <typename T>
    typename TEnableIf<TIsDerivedFrom<T, UObject>::IsDerived, const T&>::Type Get() const
	{
		check(Memory != nullptr);
		check(Struct != nullptr);
		check(Struct->IsChildOf(T::StaticClass()));
		return *((T*)Memory);
	}

	template <typename T>
	typename TEnableIf<TIsDerivedFrom<T, UObject>::IsDerived, T&>::Type GetMutable() const
	{
		check(Memory != nullptr);
		check(Struct != nullptr);
		check(Struct->IsChildOf(T::StaticClass()));
		return *((T*)Memory);
	}

	template <typename T>
	typename TEnableIf<TIsDerivedFrom<T, UObject>::IsDerived, const T*>::Type GetPtr() const
	{
		// If Memory is set, expect Struct too. Otherwise, let nulls pass through.
		check(!Memory || (Memory && Struct));
		check(!Struct || Struct->IsChildOf(T::StaticClass()));
		return ((T*)Memory);
	}

	template <typename T>
    typename TEnableIf<TIsDerivedFrom<T, UObject>::IsDerived, T*>::Type GetMutablePtr() const
	{
		// If Memory is set, expect Struct too. Otherwise, let nulls pass through.
		check(!Memory || (Memory && Struct));
		check(!Struct || Struct->IsChildOf(T::StaticClass()));
		return ((T*)Memory);
	}

	/**
	 * Struct getters (reference & pointer, const & mutable)
	 */
	template <typename T>
	typename TEnableIf<!TIsDerivedFrom<T, UObject>::IsDerived && !TIsIInterface<T>::Value, const T&>::Type Get() const
	{
		check(Memory != nullptr);
		check(Struct != nullptr);
		check(Struct->IsChildOf(T::StaticStruct()));
		return *((T*)Memory);
	}

	template <typename T>
    typename TEnableIf<!TIsDerivedFrom<T, UObject>::IsDerived && !TIsIInterface<T>::Value, T&>::Type GetMutable() const
	{
		check(Memory != nullptr);
		check(Struct != nullptr);
		check(Struct->IsChildOf(T::StaticStruct()));
		return *((T*)Memory);
	}

	template <typename T>
    typename TEnableIf<!TIsDerivedFrom<T, UObject>::IsDerived && !TIsIInterface<T>::Value, const T*>::Type GetPtr() const
	{
		// If Memory is set, expect Struct too. Otherwise, let nulls pass through.
		check(!Memory || (Memory && Struct));
		check(!Struct || Struct->IsChildOf(T::StaticStruct()));
		return ((T*)Memory);
	}

	template <typename T>
    typename TEnableIf<!TIsDerivedFrom<T, UObject>::IsDerived && !TIsIInterface<T>::Value, T*>::Type GetMutablePtr() const
	{
		// If Memory is set, expect Struct too. Otherwise, let nulls pass through.
		check(!Memory || (Memory && Struct));
		check(!Struct || Struct->IsChildOf(T::StaticStruct()));
		return ((T*)Memory);
	}

	/**
	 * IInterface getters (reference & pointer, const & mutable)
	 */
	template <typename T>
	typename TEnableIf<TIsIInterface<T>::Value, const T&>::Type Get() const
	{
		check(!Memory || (Memory && Struct));
		check(Struct->IsChildOf(UObject::StaticClass()) && ((UClass*)Struct)->ImplementsInterface(T::UClassType::StaticClass()));
		return *(T*)((UObject*)Memory)->GetInterfaceAddress(T::UClassType::StaticClass());
	}

	template <typename T>
    typename TEnableIf<TIsIInterface<T>::Value, T&>::Type GetMutable() const
	{
		check(!Memory || (Memory && Struct));
		check(Struct->IsChildOf(UObject::StaticClass()) && ((UClass*)Struct)->ImplementsInterface(T::UClassType::StaticClass()));
		return *(T*)((UObject*)Memory)->GetInterfaceAddress(T::UClassType::StaticClass());
	}

	template <typename T>
    typename TEnableIf<TIsIInterface<T>::Value, const T*>::Type GetPtr() const
	{
		check(!Memory || (Memory && Struct));
		check(Struct->IsChildOf(UObject::StaticClass()) && ((UClass*)Struct)->ImplementsInterface(T::UClassType::StaticClass()));
		return (T*)((UObject*)Memory)->GetInterfaceAddress(T::UClassType::StaticClass());
	}

	template <typename T>
    typename TEnableIf<TIsIInterface<T>::Value, T*>::Type GetMutablePtr() const
	{
		check(!Memory || (Memory && Struct));
		check(Struct->IsChildOf(UObject::StaticClass()) && ((UClass*)Struct)->ImplementsInterface(T::UClassType::StaticClass()));
		return (T*)((UObject*)Memory)->GetInterfaceAddress(T::UClassType::StaticClass());
	}

	/** @return Struct describing the data type. */
	const UStruct* GetStruct() const
	{
		return Struct;
	}

	/** @return Raw const pointer to the data. */
	const void* GetMemory() const
	{
		return Memory;
	}

	/** @return Raw mutable pointer to the data. */
	void* GetMutableMemory() const
	{
		return Memory;
	}
	
protected:
	/** UClass or UScriptStruct of the data. */
	const UStruct* Struct = nullptr;

	/** Memory pointing at the class or struct */
	void* Memory = nullptr;
};
