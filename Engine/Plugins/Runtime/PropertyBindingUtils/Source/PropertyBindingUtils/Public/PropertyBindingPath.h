// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingBindableStructDescriptor.h"
#include "PropertyBindingTypes.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "PropertyBindingBindableStructDescriptor.h"
#include "PropertyBindingDataView.h"

#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

#include "PropertyBindingPath.generated.h"

struct FPropertyBindingDataView;
struct FPropertyBindingPath;

#define UE_API PROPERTYBINDINGUTILS_API

/**
 * Struct describing an indirection at specific segment at path.
 * Returned by FPropertyBindingPath::ResolveIndirections() and FPropertyBindingPath::ResolveIndirectionsWithValue().
 * Generally there's one indirection per FProperty. Containers have one path segment but two indirection (container + inner type).
 */
struct FPropertyBindingPathIndirection
{
	FPropertyBindingPathIndirection() = default;
	explicit FPropertyBindingPathIndirection(const UStruct* InContainerStruct)
		: ContainerStruct(InContainerStruct)
	{
	}

	const FProperty* GetProperty() const
	{
		return Property;
	}

	const void* GetContainerAddress() const
	{
		return ContainerAddress;
	}

	const UStruct* GetInstanceStruct() const
	{
		return InstanceStruct;
	}

	const UStruct* GetContainerStruct() const
	{
		return ContainerStruct;
	}

	int32 GetArrayIndex() const
	{
		return ArrayIndex;
	}

	int32 GetPropertyOffset() const
	{
		return PropertyOffset;
	}

	int32 GetPathSegmentIndex() const
	{
		return PathSegmentIndex;
	}

	EPropertyBindingPropertyAccessType GetAccessType() const
	{
		return AccessType;
	}
	
	const void* GetPropertyAddress() const
	{
		return (uint8*)ContainerAddress + PropertyOffset;
	}

	void* GetMutablePropertyAddress() const
	{
		return (uint8*)ContainerAddress + PropertyOffset;
	}

#if WITH_EDITORONLY_DATA
	FName GetDisplayName() const
	{
		return DisplayName;
	}

	FName GetRedirectedName() const
	{
		return RedirectedName;
	}
	
	FGuid GetPropertyGuid() const
	{
		return PropertyGuid;
	}
#endif
	
private:
	/** Property at the indirection. */
	const FProperty* Property = nullptr;
	
	/** Address of the container class/struct where the property belongs to. Only valid if created with ResolveIndirectionsWithValue() */
	const void* ContainerAddress = nullptr;
	
	/** Type of the container class/struct. */
	const UStruct* ContainerStruct = nullptr;
	
	/** Type of the instance class/struct of when AccessType is ObjectInstance or StructInstance. */
	const UStruct* InstanceStruct = nullptr;

	/** Array index for static and dynamic arrays. Note: static array indexing is baked in the PropertyOffset. */
	int32 ArrayIndex = INDEX_NONE;
	
	/** Offset of the property relative to ContainerAddress. Includes static array indexing. */
	int32 PropertyOffset = INDEX_NONE;
	
	/** Index of the path segment where indirection originated from. */
	int32 PathSegmentIndex = INDEX_NONE;
	
	/** How to access the data through the indirection. */
	EPropertyBindingPropertyAccessType AccessType = EPropertyBindingPropertyAccessType::Offset;

#if WITH_EDITORONLY_DATA
	FName DisplayName;

	/** Redirected name, if the give property name was not found but was reconciled using core redirect or property Guid. Requires ResolveIndirections/WithValue() to be called with bHandleRedirects = true. */
	FName RedirectedName;

	/** Guid of the property for Blueprint classes or User Defined Structs. Requires ResolveIndirections/WithValue() to be called with bHandleRedirects = true. */
	FGuid PropertyGuid;
#endif

	friend FPropertyBindingPath;
};

/** Struct describing a path segment in FPropertyBindingPath. */
USTRUCT()
struct FPropertyBindingPathSegment
{
	GENERATED_BODY()

	FPropertyBindingPathSegment() = default;

	explicit FPropertyBindingPathSegment(const FName InName, const int32 InArrayIndex = INDEX_NONE)
		: Name(InName)
		, ArrayIndex(InArrayIndex)
	{
	}

	explicit FPropertyBindingPathSegment(const FName InName, const int32 InArrayIndex, const UStruct* InInstanceStruct, const EPropertyBindingPropertyAccessType InAccessType)
		: Name(InName)
		, ArrayIndex(InArrayIndex)
		, InstanceStruct(InInstanceStruct)
		, InstancedStructAccessType(InAccessType)
	{
	}

	bool operator==(const FPropertyBindingPathSegment& RHS) const
	{
		return Name == RHS.Name && InstanceStruct == RHS.InstanceStruct && ArrayIndex == RHS.ArrayIndex;
	}

	bool operator!=(const FPropertyBindingPathSegment& RHS) const
	{
		return !(*this == RHS);
	}

	void SetName(const FName InName)
	{
		Name = InName;
	}

	FName GetName() const
	{
		return Name;
	}

	void SetArrayIndex(const int32 InArrayIndex)
	{
		ArrayIndex = InArrayIndex;
	}

	int32 GetArrayIndex() const
	{
		return ArrayIndex;
	}

	void SetInstanceStruct(const UStruct* InInstanceStruct, const EPropertyBindingPropertyAccessType InAccessType = EPropertyBindingPropertyAccessType::StructInstance)
	{
		InstanceStruct = InInstanceStruct;
		InstancedStructAccessType = InAccessType;
	}

	const UStruct* GetInstanceStruct() const
	{
		return InstanceStruct;
	}

	EPropertyBindingPropertyAccessType GetInstancedStructAccessType() const
	{
		return InstancedStructAccessType;
	}

#if WITH_EDITORONLY_DATA
	FName GetDisplayName() const
	{
		return DisplayName;
	}

	void SetDisplayName(const FName InName)
	{
		DisplayName = InName;
	}

	FGuid GetPropertyGuid() const
	{
		return PropertyGuid;
	}

	void SetPropertyGuid(const FGuid NewGuid)
	{
		PropertyGuid = NewGuid;
	}
#endif
	
private:
	/** Name of the property */
	UPROPERTY(VisibleAnywhere, Category = "Bindings")
	FName Name;

	/** Array index if the property is dynamic or static array. */
	UPROPERTY(VisibleAnywhere, Category = "Bindings")
	int32 ArrayIndex = INDEX_NONE;

	/** Type of the instanced struct or object reference by the property at the segment. This allows the path to be resolved when it points to a specific instance. */
	UPROPERTY(VisibleAnywhere, Category = "Bindings")
	TObjectPtr<const UStruct> InstanceStruct = nullptr;

	UPROPERTY(VisibleAnywhere, Category = "Bindings")
	EPropertyBindingPropertyAccessType InstancedStructAccessType = EPropertyBindingPropertyAccessType::Unset;

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Category = "Bindings")
	FName DisplayName;

	/** Guid of the property for Blueprint classes, User Defined Structs, or Property Bags. */
	UPROPERTY(transient, VisibleAnywhere, Category = "Bindings", AdvancedDisplay)
	FGuid PropertyGuid;
#endif
};


/**
 * Representation of a property path that can be used for property access and binding.
 *
 * The engine supports many types of property paths, this implementation has these specific properties:
 *		- Allow to resolve all the indirections from a base value (object or struct) up to the leaf property
 *		- handle redirects from Core Redirect, BP classes, User Defines Structs and Property Bags
 *
 * You may also take a look at: FCachedPropertyPath, TFieldPath<>, FPropertyPath.
 */
USTRUCT()
struct FPropertyBindingPath
{
	GENERATED_BODY()

	FPropertyBindingPath() = default;

#if WITH_EDITORONLY_DATA
	explicit FPropertyBindingPath(const FGuid InStructID)
		: StructID(InStructID)
	{
	}

	explicit FPropertyBindingPath(const FGuid InStructID, const FName PropertyName)
		: StructID(InStructID)
	{
		Segments.Emplace(PropertyName);
	}
	
	explicit FPropertyBindingPath(const FGuid InStructID, TConstArrayView<FPropertyBindingPathSegment> InSegments)
		: StructID(InStructID)
		, Segments(InSegments)
	{
	}
#endif // WITH_EDITORONLY_DATA
	
	/**
	 * Parses path from string. The path should be in format: Foo.Bar[1].Baz
	 * @param InPath Path string to parse
	 * @return true if path was parsed successfully.
	 */
	UE_API bool FromString(const FStringView InPath);

	/**
	 * Returns the property path as a one string. Highlight allows to decorate a specific segment.
	 * @param HighlightedSegment Index of the highlighted path segment
	 * @param HighlightPrefix String to append before highlighted segment
	 * @param HighlightPostfix String to append after highlighted segment
	 * @param bOutputInstances if true, the instance struct types will be output. 
	 * @param FirstSegment Index of the first path segment to be stringified.
	 */
	UE_API FString ToString(const int32 HighlightedSegment = INDEX_NONE, const TCHAR* HighlightPrefix = nullptr, const TCHAR* HighlightPostfix = nullptr, const bool bOutputInstances = false, const int32 FirstSegment = 0) const;

	/**
	 * Resolves the property path against base struct type. The path is assumed to be relative to the BaseStruct.
	 * @param BaseStruct Base struct/class type the path is relative to.
	 * @param OutIndirections Indirections describing how the properties were accessed. 
	 * @param OutError Optional, pointer to string where error will be logged if update fails.
	 * @param bHandleRedirects If true, the method will try to resolve missing properties using core redirects, and properties on Blueprint and User Defined Structs by ID. Available only in editor builds!
	 * @param bResetIndirectionsIfFailed If true, indirections resolved so far will be cleared if the path failed to resolve in the end. 
	 * @return true of path could be resolved against the base value, and instance types were updated.
	 */
	UE_API bool ResolveIndirections(const UStruct* BaseStruct, TArray<FPropertyBindingPathIndirection>& OutIndirections, FString* OutError = nullptr, bool bHandleRedirects = false, bool bResetIndirectionsIfFailed = true) const;
	UE_API bool ResolveIndirections(const UStruct* BaseStruct, TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>>& OutIndirections, FString* OutError = nullptr, bool bHandleRedirects = false, bool bResetIndirectionsIfFailed = true) const;

	/**
	 * Resolves the property path against base value. The path is assumed to be relative to the BaseValueView.
	 * @param BaseValueView Base value the path is relative to.
	 * @param OutIndirections Indirections describing how the properties were accessed. 
	 * @param OutError Optional, pointer to string where error will be logged if update fails.
	 * @param bHandleRedirects If true, the method will try to resolve missing properties using core redirects, and properties on Blueprint and User Defined Structs by ID. Available only in editor builds!
	 * @param bResetIndirectionsIfFailed If true, indirections resolved so far will be cleared if the path failed to resolve in the end.
	 * @return true of path could be resolved against the base value, and instance types were updated.
	 */
	UE_API bool ResolveIndirectionsWithValue(const FPropertyBindingDataView BaseValueView, TArray<FPropertyBindingPathIndirection>& OutIndirections, FString* OutError = nullptr, bool bHandleRedirects = false, bool bResetIndirectionsIfFailed = true) const;
	UE_API bool ResolveIndirectionsWithValue(const FPropertyBindingDataView BaseValueView, TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>>& OutIndirections, FString* OutError = nullptr, bool bHandleRedirects = false, bool bResetIndirectionsIfFailed = true) const;

	/**
	 * Updates property segments from base struct type. The path is expected to be relative to the BaseStruct.
	 * The method handles renamed properties (core redirect, Blueprint, User Defined Structs and Property Bags by ID).
	 * @param BaseStruct Base struct/class type the path is relative to
	 * @param OutError Optional, pointer to string where error will be logged if update fails.
	 * @return true of path could be resolved against the base value, and instance types were updated.
	 */
	UE_API bool UpdateSegments(const UStruct* BaseStruct, FString* OutError = nullptr);

	/**
	 * Updates property segments from base value. The path is expected to be relative to the base value.
	 * The method updates instance types, and handles renamed properties (core redirect, Blueprint, User Defined Structs and Property Bags by ID).
	 * By storing the instance types on the path, we can resolve the path without the base value later.
	 * @param BaseValueView Base value the path is relative to.
	 * @param OutError Optional, pointer to string where error will be logged if update fails.
	 * @return true of path could be resolved against the base value, and instance types were updated.
	 */
	UE_API bool UpdateSegmentsFromValue(const FPropertyBindingDataView BaseValueView, FString* OutError = nullptr);

	/**
	 * Updates property segments from resolved indirections. The path is expected to be relative to the base value.
	 * The method updates instance types, and handles renamed properties (core redirect, Blueprint, User Defined Structs and Property Bags by ID).
	 * By storing the instance types on the path, we can resolve the path without the base value later.
	 * @param Indirections completely resolved indirections
	 */
	UE_API void UpdateSegmentsFromIndirections(const TConstArrayView<FPropertyBindingPathIndirection>& Indirections);

	/**
	 * Check if the binding path contains any structs from cached instance struct type.
	 * @param InStructs any structs the binding path may contain
	 * @return true if path contains any struct
	 */
	UE_API bool ContainsAnyStruct(TConstArrayView<const UStruct*> InStructs) const;

	/**
	 * Check if the binding path contains any structs from base struct type.
	 * @param InStructs any structs the binding path may contain
	 * @param BaseStruct Base struct/class type the path is relative to
	 * @return true if path contains any struct
	 */
	UE_API bool ContainsAnyStruct(TConstArrayView<const UStruct*> InStructs, const UStruct* BaseStruct) const;

	/**
	 * Check if the binding path contains any structs from base value.
	 * @param InStructs any structs the binding path may contain
	 * @param BaseValueView Base value the path is relative to
	 * @return true if path contains any struct
	 */
	UE_API bool ContainsAnyStructWithValue(TConstArrayView<const UStruct*> InStructs, const FPropertyBindingDataView BaseValueView) const;

	/**
	 * Check if indirections contain any structs. 
	 * @param InStructs any structs the binding path may contain
	 * @param Indirections Indirections resolved so far to check against
	 * @return true if indirections contains any struct
	 */
	UE_API static bool IndirectionsContainAnyStruct(TConstArrayView<const UStruct*> InStructs, const TConstArrayView<FPropertyBindingPathIndirection> Indirections);

	/** @return true if the path is empty. In that case the path points to the struct. */
	bool IsPathEmpty() const
	{
		return Segments.IsEmpty();
	}

	/** @return true if any of the path segments is and indirection via instanced struct or object. */
	bool HasAnyInstancedIndirection() const
	{
		return Segments.ContainsByPredicate([](const FPropertyBindingPathSegment& Segment)
		{
			return Segment.GetInstanceStruct() != nullptr;
		});
	}

	/** Reset the path to empty. */
	void Reset()
	{
#if WITH_EDITORONLY_DATA
		StructID = FGuid();
#endif
		Segments.Reset();
	}

#if WITH_EDITORONLY_DATA
	const FGuid& GetStructID() const
	{
		return StructID;
	}
	
	void SetStructID(const FGuid NewStructID)
	{
		StructID = NewStructID;
	}
#endif // WITH_EDITORONLY_DATA

	TConstArrayView<FPropertyBindingPathSegment> GetSegments() const
	{
		return Segments;
	}
	
	TArrayView<FPropertyBindingPathSegment> GetMutableSegments()
	{
		return Segments;
	}
	
	int32 NumSegments() const
	{
		return Segments.Num();
	}
	
	const FPropertyBindingPathSegment& GetSegment(const int32 Index) const
	{
		return Segments[Index];
	}

	/** Adds a path segment to the path. */
	void AddPathSegment(const FName InName
		, const int32 InArrayIndex = INDEX_NONE
		, const UStruct* InInstanceType = nullptr
		, EPropertyBindingPropertyAccessType InInstanceAccessType = EPropertyBindingPropertyAccessType::StructInstance)
	{
		Segments.Emplace(InName, InArrayIndex, InInstanceType, InInstanceAccessType);
	}

	/** Adds a path segment to the path. */
	void AddPathSegment(const FPropertyBindingPathSegment& PathSegment)
	{
		Segments.Add(PathSegment);
	}

	/** Test if paths are equal. */
	UE_API bool operator==(const FPropertyBindingPath& RHS) const;

	/**
	 * Test if this paths includes the provided path.
	 * A path includes another one when they are == but this path can be longer.
	 */
	UE_API bool Includes(const FPropertyBindingPath& Other) const;

private:
#if WITH_EDITORONLY_DATA
	/** ID of the struct this property path is relative to. */
	UPROPERTY(VisibleAnywhere, Category = "Bindings", AdvancedDisplay)
	FGuid StructID;
#endif // WITH_EDITORONLY_DATA

	/** Path segments pointing to a specific property on the path. */
	UPROPERTY(VisibleAnywhere, Category = "Bindings")
	TArray<FPropertyBindingPathSegment> Segments;

	template<typename Allocator>
	bool ResolveIndirectionsWithValue(const FPropertyBindingDataView BaseValueView, TArray<FPropertyBindingPathIndirection, Allocator>& OutIndirections, FString* OutError = nullptr, bool bHandleRedirects = false, bool bResetIndirectionsIfFailed = true) const;
};


/**
 * Describes how the copy should be performed.
 */
UENUM()
enum class EPropertyCopyType : uint8
{
	None,						// No copying
	
	CopyPlain,					// For plain old data types, we do a simple memcpy.
	CopyComplex,				// For more complex data types, we need to call the properties copy function
	CopyBool,					// Read and write properties using bool property helpers, as source/dest could be bitfield or boolean
	CopyStruct,					// Use struct copy operation, as this needs to correctly handle CPP struct ops
	CopyObject,					// Read and write properties using object property helpers, as source/dest could be regular/weak/soft etc.
	CopyName,					// FName needs special case because its size changes between editor/compiler and runtime.
	CopyFixedArray,				// Array needs special handling for fixed size TArrays

	StructReference,			// Copies pointer to a source struct into a given struct specified by FPropertyBindingBindingCollection::PropertyReferenceStructType.

	/* Promote the type during the copy */

	/* Bool promotions */
	PromoteBoolToByte,
	PromoteBoolToInt32,
	PromoteBoolToUInt32,
	PromoteBoolToInt64,
	PromoteBoolToFloat,
	PromoteBoolToDouble,

	/* Byte promotions */
	PromoteByteToInt32,
	PromoteByteToUInt32,
	PromoteByteToInt64,
	PromoteByteToFloat,
	PromoteByteToDouble,

	/* Int32 promotions */
	PromoteInt32ToInt64,
	PromoteInt32ToFloat,	// This is strictly sketchy because of potential data loss, but it is usually OK in the general case
	PromoteInt32ToDouble,

	/* UInt32 promotions */
	PromoteUInt32ToInt64,
	PromoteUInt32ToFloat,	// This is strictly sketchy because of potential data loss, but it is usually OK in the general case
	PromoteUInt32ToDouble,

	/* Float promotions */
	PromoteFloatToInt32,
	PromoteFloatToInt64,
	PromoteFloatToDouble,

	/* Double promotions */
	DemoteDoubleToInt32,
	DemoteDoubleToInt64,
	DemoteDoubleToFloat,
};

/**
 * Used internally.
 * Property indirection is a resolved property path segment, used for accessing properties in structs.
 */
USTRUCT()
struct FPropertyBindingPropertyIndirection
{
	GENERATED_BODY()

	/** Index in the array the property points at. */
	UPROPERTY()
	FPropertyBindingIndex16 ArrayIndex;

	/** Cached offset of the property */
	UPROPERTY()
	uint16 Offset = 0;

	/** Cached offset of the property */
	UPROPERTY()
	FPropertyBindingIndex16 NextIndex;

	/** Type of access/indirection. */
	UPROPERTY()
	EPropertyBindingPropertyAccessType Type = EPropertyBindingPropertyAccessType::Offset;

	/** Type of the struct or object instance in case the segment is pointing into an instanced data. */
	UPROPERTY()
	TObjectPtr<const UStruct> InstanceStruct = nullptr;

	/** Cached array property. */
	const FArrayProperty* ArrayProperty = nullptr;

	friend uint32 GetTypeHash(const FPropertyBindingPropertyIndirection& Indirection)
	{
		uint32 Hash = GetTypeHash(Indirection.ArrayIndex);
		Hash = HashCombineFast(Hash, GetTypeHash(Indirection.Offset));
		Hash = HashCombineFast(Hash, GetTypeHash(Indirection.NextIndex));
		Hash = HashCombineFast(Hash, GetTypeHash(Indirection.Type));
		Hash = HashCombineFast(Hash, GetTypeHash(Indirection.InstanceStruct));
		
		return Hash;
	}
};

/**
 * Used internally.
 * Describes property copy, the property from source is copied into the property at the target.
 * Copy target struct is described in the property copy batch.
 */
USTRUCT()
struct FPropertyBindingCopyInfo
{
	GENERATED_BODY()

	/** Source property access. */
	UPROPERTY()
	FPropertyBindingPropertyIndirection SourceIndirection;

	/** Target property access. */
	UPROPERTY()
	FPropertyBindingPropertyIndirection TargetIndirection;

	/** Describes how to get the source data pointer for the copy. */
	UPROPERTY()
	FInstancedStruct SourceDataHandle;

	/** Cached pointer to the leaf property of the access. */
	const FProperty* SourceLeafProperty = nullptr;

	/** Cached pointer to the leaf property of the access. */
	const FProperty* TargetLeafProperty = nullptr;

	/** Type of the source data, used for validation. */
	UPROPERTY(Transient)
	TObjectPtr<const UStruct> SourceStructType = nullptr;

	/** Cached property element size * dim. */
	UPROPERTY()
	int32 CopySize = 0;

	/** Type of the copy */
	UPROPERTY()
	EPropertyCopyType Type = EPropertyCopyType::None;

	/**
	 * The source property and the target property are array properties.
	 * The inner type of the source property can be promoted to the inner type of the target property.
	 */
	UPROPERTY()
	bool bPromoteArrayCopy = false;

	/** Copy will be performed from target property to the source property. */
	UPROPERTY()
	bool bCopyFromTargetToSource = false;
};

/**
 * Describes a batch of property copies from many sources to one target struct.
 * Note: The batch is used to reference both bindings and copies (a binding turns into copy when resolved).
 */
USTRUCT()
struct FPropertyBindingCopyInfoBatch
{
	GENERATED_BODY()

	/** Expected target struct */
	UPROPERTY()
	TInstancedStruct<FPropertyBindingBindableStructDescriptor> TargetStruct;

	/** Index to first binding/copy. */
	UPROPERTY()
	FPropertyBindingIndex16 BindingsBegin;

	/** Index to one past the last binding/copy. */
	UPROPERTY()
	FPropertyBindingIndex16 BindingsEnd;

	/** Index to first property function. */
	UPROPERTY()
	FPropertyBindingIndex16 PropertyFunctionsBegin;

	/** Index to one past the last property function. */
	UPROPERTY()
	FPropertyBindingIndex16 PropertyFunctionsEnd;
};

#undef UE_API
