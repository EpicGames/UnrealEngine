// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"

#include "PropertyBindingTypes.generated.h"

PROPERTYBINDINGUTILS_API DECLARE_LOG_CATEGORY_EXTERN(LogPropertyBindingUtils, Warning, All);

#ifndef WITH_PROPERTYBINDINGUTILS_DEBUG
#define WITH_PROPERTYBINDINGUTILS_DEBUG (!(UE_BUILD_SHIPPING || UE_BUILD_SHIPPING_WITH_EDITOR || UE_BUILD_TEST) && 1)
#endif // WITH_PROPERTYBINDINGUTILS_DEBUG

UENUM()
enum class UE_DEPRECATED(5.6, "Use EPropertyBindingPropertyAccessType instead") EPropertyBindingAccessType : uint8
{
	Offset,			// Access node is a simple basePtr + offset
	Object,			// Access node needs to dereference an object at its current address
	WeakObject,		// Access is a weak object
	SoftObject,		// Access is a soft object
	ObjectInstance,	// Access node needs to dereference an object of specific type at its current address
	StructInstance,	// Access node needs to dereference an instanced struct of specific type at its current address
	IndexArray,		// Access node indexes a dynamic array
};

UENUM()
enum class EPropertyBindingPropertyAccessType : uint8
{
	Offset,						// Access node is a simple basePtr + offset
	Object,						// Access node needs to dereference an object at its current address
	WeakObject,					// Access is a weak object
	SoftObject,					// Access is a soft object
	ObjectInstance,				// Access node needs to dereference an object of specific type at its current address
	StructInstance,				// Access node needs to dereference an instanced struct of specific type at its current address
	IndexArray,					// Access node indexes a dynamic array
	SharedStruct,				// Access node needs to dereference a shared struct of specific type at its current address
	StructInstanceContainer,	// Access node needs to dereference an instanced struct container at its current address
	Unset
};

/** uint16 index that can be invalid. */
USTRUCT(BlueprintType)
struct FPropertyBindingIndex16
{
	GENERATED_BODY()

	static constexpr uint16 InvalidValue = MAX_uint16;
	static const FPropertyBindingIndex16 Invalid;

	friend inline uint32 GetTypeHash(const FPropertyBindingIndex16 Index)
	{
		return GetTypeHash(Index.Value);
	}

	/** @return true if the given index can be represented by the type. */
	static bool IsValidIndex(const int32 Index)
	{
		return Index >= 0 && Index < static_cast<int32>(MAX_uint16);
	}

	FPropertyBindingIndex16() = default;

	/**
	 * Construct from a uint16 index where MAX_uint16 is considered an invalid index
	 * (i.e., FPropertyBindingIndex16::InvalidValue).
	 */
	explicit FPropertyBindingIndex16(const uint16 InIndex) : Value(InIndex)
	{
	}

	/**
	 * Construct from an int32 index where INDEX_NONE is considered an invalid index
	 * and converted to FPropertyBindingIndex16::InvalidValue (i.e., MAX_uint16).
	 */
	explicit FPropertyBindingIndex16(const int32 InIndex)
	{
		check(InIndex == INDEX_NONE || IsValidIndex(InIndex));
		Value = InIndex == INDEX_NONE ? InvalidValue : static_cast<uint16>(InIndex);
	}

	/** @return value of the index or FPropertyBindingIndex16::InvalidValue (i.e. MAX_uint16) if invalid. */
	uint16 Get() const
	{
		return Value;
	}
	
	/** @return the index value as int32, mapping invalid value to INDEX_NONE. */
	int32 AsInt32() const
	{
		return Value == InvalidValue ? INDEX_NONE : Value;
	}

	/** @return true if the index is valid. */
	bool IsValid() const
	{
		return Value != InvalidValue;
	}

	bool operator==(const FPropertyBindingIndex16& RHS) const
	{
		return Value == RHS.Value;
	}

	bool operator!=(const FPropertyBindingIndex16& RHS) const
	{
		return Value != RHS.Value;
	}

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

protected:
	UPROPERTY()
	uint16 Value = InvalidValue;
};

template<>
struct TStructOpsTypeTraits<FPropertyBindingIndex16>
	: TStructOpsTypeTraitsBase2<FPropertyBindingIndex16>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};


namespace UE::PropertyBinding
{
#if WITH_EDITORONLY_DATA
	/**
	 * List of external types conversion functors that can be used to
	 * convert compatible struct types to FPropertyBindingIndex16 in FPropertyBindingIndex16::SerializeFromMismatchedTag
	 */
	extern PROPERTYBINDINGUTILS_API TArray<TFunction<bool(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, TNotNull<FPropertyBindingIndex16*> Index)>> PropertyBindingIndex16ConversionFuncList;
#endif

/** Enum describing property compatibility */
enum class EPropertyCompatibility : uint8
{
	/** Properties are incompatible */
	Incompatible,
	/** Properties are directly compatible */
	Compatible,
	/** Properties can be copied with a simple type promotion */
	Promotable,
};

/** Struct of parameters used to create a property in a property bag. */
struct FPropertyCreationDescriptor
{
	/** Property Bag Description of the Property to Create */
	FPropertyBagPropertyDesc PropertyDesc;

	/** Optional: property to copy into the new created property */
	const FProperty* SourceProperty = nullptr;

	/** Optional: container address of the property to copy */
	const void* SourceContainerAddress = nullptr;
};

/** @return how properties are compatible for copying. */
PROPERTYBINDINGUTILS_API EPropertyCompatibility GetPropertyCompatibility(const FProperty* FromProperty, const FProperty* ToProperty);

/**
 * Helper function to
 * 1. Generate unique names for the incoming property descriptors (to avoid changing the existing properties in the property bag)
 * 2. Add uniquely named properties to the property bag
 * 3. Attempt to copy values from the Source Property / Address og the property descriptors.
 */
PROPERTYBINDINGUTILS_API void CreateUniquelyNamedPropertiesInPropertyBag(TArrayView<FPropertyCreationDescriptor> InOutCreationDescs, FInstancedPropertyBag& OutPropertyBag);

} // UE::PropertyBinding

