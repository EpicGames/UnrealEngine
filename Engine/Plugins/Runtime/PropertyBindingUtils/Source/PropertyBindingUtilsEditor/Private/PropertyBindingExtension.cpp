// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBindingExtension.h"

#include "Containers/StridedView.h"
#include "DetailLayoutBuilder.h"
#include "EdGraphSchema_K2.h"
#include "Features/IModularFeatures.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "IPropertyAccessEditor.h"
#include "IPropertyUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PropertyBagDetails.h"
#include "PropertyBindingBindableStructDescriptor.h"
#include "PropertyBindingBinding.h"
#include "PropertyBindingBindingCollectionOwner.h"
#include "PropertyBindingDataView.h"
#include "PropertyBindingPath.h"
#include "PropertyBindingTypes.h"
#include "ScopedTransaction.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/InstancedStructContainer.h"
#include "Styling/AppStyle.h"
#include "UObject/WeakInterfacePtr.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "PropertyBindingExtension"

struct FStateTreeBlueprintPropertyRef;
struct FStateTreeEditorNode;

namespace UE::PropertyBinding
{
/** Helper struct to Begin/End Sections */
struct FMenuSectionHelper
{
	PROPERTYBINDINGUTILSEDITOR_API explicit FMenuSectionHelper(FMenuBuilder& MenuBuilder);
	PROPERTYBINDINGUTILSEDITOR_API ~FMenuSectionHelper();

	PROPERTYBINDINGUTILSEDITOR_API void SetSection(const FText& InSection);

private:
	FText CurrentSection;
	FMenuBuilder& MenuBuilder;
	bool bSectionOpened = false;
};

IPropertyBindingBindingCollectionOwner* FindBindingsOwner(UObject* InObject)
{
	IPropertyBindingBindingCollectionOwner* Result = nullptr;

	for (UObject* Outer = InObject; Outer; Outer = Outer->GetOuter())
	{
		if (IPropertyBindingBindingCollectionOwner* BindingOwner = Cast<IPropertyBindingBindingCollectionOwner>(Outer))
		{
			Result = BindingOwner;
			break;
		}
	}
	return Result;
}

UStruct* ResolveLeafValueStructType(FPropertyBindingDataView ValueView, TConstArrayView<FBindingChainElement> InBindingChain)
{
	if (ValueView.GetMemory() == nullptr)
	{
		return nullptr;
	}
	
	FPropertyBindingPath Path;

	for (const FBindingChainElement& Element : InBindingChain)
	{
		if (const FProperty* Property = Element.Field.Get<FProperty>())
		{
			Path.AddPathSegment(Property->GetFName(), Element.ArrayIndex);
		}
		else if (Element.Field.Get<UFunction>())
		{
			// Cannot handle function calls
			return nullptr;
		}
	}

	TArray<FPropertyBindingPathIndirection> Indirections;
	if (!Path.ResolveIndirectionsWithValue(ValueView, Indirections)
		|| Indirections.IsEmpty())
	{
		return nullptr;
	}

	// Last indirection points to the value of the leaf property, check the type.
	const FPropertyBindingPathIndirection& LastIndirection = Indirections.Last();

	UStruct* Result = nullptr;

	if (LastIndirection.GetContainerAddress())
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(LastIndirection.GetProperty()))
		{
			// Get the type of the instanced struct's value.
			if (StructProperty->Struct == TBaseStructure<FInstancedStruct>::Get())
			{
				const FInstancedStruct& InstancedStruct = *static_cast<const FInstancedStruct*>(LastIndirection.GetPropertyAddress());
				Result = const_cast<UScriptStruct*>(InstancedStruct.GetScriptStruct());
			}
		}
		else if (CastField<FObjectProperty>(LastIndirection.GetProperty()))
		{
			// Get type of the instanced object.
			if (const UObject* Object = *static_cast<UObject* const*>(LastIndirection.GetPropertyAddress()))
			{
				Result = Object->GetClass();
			}
		}
	}

	return Result;
}

void MakeStructPropertyPathFromBindingChain(const FGuid StructID, TConstArrayView<FBindingChainElement> InBindingChain, FPropertyBindingDataView DataView, FPropertyBindingPath& OutPath)
{
	OutPath.Reset();
	OutPath.SetStructID(StructID);
	
	for (const FBindingChainElement& Element : InBindingChain)
	{
		if (const FProperty* Property = Element.Field.Get<FProperty>())
		{
			OutPath.AddPathSegment(Property->GetFName(), Element.ArrayIndex);
		}
		else if (const UFunction* Function = Element.Field.Get<UFunction>())
		{
			OutPath.AddPathSegment(Function->GetFName());
		}
	}

	OutPath.UpdateSegmentsFromValue(DataView);
}

TSharedPtr<const IPropertyHandle> MakeStructPropertyPathFromPropertyHandle(
	const TSharedPtr<const IPropertyHandle>& InPropertyHandle
	, FPropertyBindingPath& OutPath
	, const FGuid InFallbackStructID)
{
	OutPath.Reset();

	FGuid StructID;
	TArray<FPropertyBindingPathSegment> PathSegments;

	auto ExtractStructIdFromPropertyHandle = [&StructID](const TSharedPtr<const IPropertyHandle>& ParentHandle)
	{
		if (const FString* IDString = ParentHandle->GetInstanceMetaData(MetaDataStructIDName))
		{
			LexFromString(StructID, **IDString);
			return true;
		}

		return false;
	};

	TSharedPtr<const IPropertyHandle> BindablePropertyHandle;
	TSharedPtr<const IPropertyHandle> CurrentPropertyHandle = InPropertyHandle;
	while (CurrentPropertyHandle.IsValid())
	{
		if (const FProperty* Property = CurrentPropertyHandle->GetProperty())
		{
			FPropertyBindingPathSegment& Segment = PathSegments.InsertDefaulted_GetRef(0); // Traversing from leaf to root, insert in reverse.

			// Store path up to the property which has ID.
			Segment.SetName(Property->GetFName());
			Segment.SetArrayIndex(CurrentPropertyHandle->GetIndexInArray());

			// Store type of the object (e.g. for instanced objects or instanced structs).
			if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				if (ObjectProperty->HasAnyPropertyFlags(CPF_PersistentInstance | CPF_InstancedReference))
				{
					const UObject* Object = nullptr;
					if (CurrentPropertyHandle->GetValue(Object) == FPropertyAccess::Success)
					{
						if (Object)
						{
							Segment.SetInstanceStruct(Object->GetClass(), EPropertyBindingPropertyAccessType::ObjectInstance);
						}
					}
				}
			}
			else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				void* Address = nullptr;
				if (CurrentPropertyHandle->GetValueData(Address) == FPropertyAccess::Success)
				{
					if (Address)
					{
						if (StructProperty->Struct == TBaseStructure<FInstancedStruct>::Get())
						{
							FInstancedStruct& Struct = *static_cast<FInstancedStruct*>(Address);
							Segment.SetInstanceStruct(Struct.GetScriptStruct(), EPropertyBindingPropertyAccessType::StructInstance);
						}
						else if (StructProperty->Struct == TBaseStructure<FSharedStruct>::Get())
						{
							FSharedStruct& Struct = *static_cast<FSharedStruct*>(Address);
							Segment.SetInstanceStruct(Struct.GetScriptStruct(), EPropertyBindingPropertyAccessType::SharedStruct);
						}
						else if (StructProperty->Struct == TBaseStructure<FInstancedStructContainer>::Get())
						{
							FInstancedStructContainer& StructContainer = *static_cast<FInstancedStructContainer*>(Address);
							check(StructContainer.IsValidIndex(Segment.GetArrayIndex()));
							const FConstStructView StructView = StructContainer[Segment.GetArrayIndex()];
							Segment.SetInstanceStruct(StructView.GetScriptStruct(), EPropertyBindingPropertyAccessType::StructInstanceContainer);
						}
					}
				}
			}

			// Array access is represented as: "Array, PropertyInArray[Index]", we're traversing from leaf to root, skip the node without index.
			// Advancing the node before ID test, since the array is on the instance data, the ID will be on the Array node.
			if (Segment.GetArrayIndex() != INDEX_NONE)
			{
				TSharedPtr<const IPropertyHandle> ParentPropertyHandle = CurrentPropertyHandle->GetParentHandle();
				if (ParentPropertyHandle.IsValid())
				{
					const FProperty* ParentProperty = ParentPropertyHandle->GetProperty();
					if (ParentProperty
						&& ParentProperty->IsA<FArrayProperty>()
						&& Property->GetFName() == ParentProperty->GetFName())
					{
						CurrentPropertyHandle = ParentPropertyHandle;
					}
				}
			}

			// Bindable property must have node ID
			if (ExtractStructIdFromPropertyHandle(CurrentPropertyHandle))
			{
				BindablePropertyHandle = CurrentPropertyHandle;
				break;
			}
		}

		TSharedPtr<IPropertyHandle> ParentHandle = CurrentPropertyHandle->GetParentHandle();
		if (ParentHandle)
		{
			// Let's check if a property is acting as the root of the property path
			if (ParentHandle->HasMetaData(MetaDataBindingRoot))
			{
				if (ExtractStructIdFromPropertyHandle(ParentHandle))
				{
					BindablePropertyHandle = CurrentPropertyHandle;
				}

				break;
			}
		}
		
		CurrentPropertyHandle = MoveTemp(ParentHandle);
	}

	if (!StructID.IsValid() && InFallbackStructID.IsValid())
	{
		StructID = InFallbackStructID;
	}

	if (StructID.IsValid())
	{
		OutPath = FPropertyBindingPath(StructID, PathSegments);
	}

	return BindablePropertyHandle;
}

bool HasMetaData(FName MetaData, const TSharedRef<const IPropertyHandle>& InPropertyHandle)
{
	TSharedPtr<const IPropertyHandle> CurrentPropertyHandle = InPropertyHandle;
	while (CurrentPropertyHandle.IsValid())
	{
		if (const FProperty* MetaDataProperty = CurrentPropertyHandle->GetMetaDataProperty())
		{
			if (const FStructProperty* StructProperty = CastField<const FStructProperty>(MetaDataProperty))
			{
				if (StructProperty->Struct->HasMetaData(MetaData))
				{
					return true;
				}
			}
			else if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(MetaDataProperty))
			{
				if (ObjectProperty->PropertyClass->HasMetaData(MetaData))
				{
					return true;
				}
			}
		}
		if (CurrentPropertyHandle->HasMetaData(MetaData))
		{
			return true;
		}
		CurrentPropertyHandle = CurrentPropertyHandle->GetParentHandle();
	}
	return false;
}

FMenuSectionHelper::FMenuSectionHelper(FMenuBuilder& MenuBuilder): MenuBuilder(MenuBuilder)
{
}

FMenuSectionHelper::~FMenuSectionHelper()
{
	if (bSectionOpened)
	{
		MenuBuilder.EndSection();
	}
}

void FMenuSectionHelper::SetSection(const FText& InSection)
{
	if (!InSection.IdenticalTo(CurrentSection))
	{
		if (bSectionOpened)
		{
			MenuBuilder.EndSection();
		}
		CurrentSection = InSection;
		MenuBuilder.BeginSection(NAME_None, CurrentSection);
		bSectionOpened = true;
	}
}

// deprecated
FText GetPropertyTypeText(const FProperty* Property)
{
	return GetPropertyTypeText(GetDefault<UEdGraphSchema_K2>(), Property);
}

FText GetPropertyTypeText(TNotNull<const UEdGraphSchema_K2*> Schema, const FProperty* Property)
{
	FEdGraphPinType PinType;
	Schema->ConvertPropertyToPinType(Property, PinType);
				
	const FName PinSubCategory = PinType.PinSubCategory;
	const UObject* PinSubCategoryObject = PinType.PinSubCategoryObject.Get();
	if (PinSubCategory != UEdGraphSchema_K2::PSC_Bitmask && PinSubCategoryObject)
	{
		if (const UField* Field = Cast<const UField>(PinSubCategoryObject))
		{
			return Field->GetDisplayNameText();
		}
		return FText::FromString(PinSubCategoryObject->GetName());
	}

	return UEdGraphSchema_K2::GetCategoryText(PinType.PinCategory, NAME_None, true);
}

TSharedRef<SWidget> MakeBindingPropertyInfoWidget(TNotNull<const UEdGraphSchema_K2*> Schema, const FText& InDisplayText, const FEdGraphPinType& InPinType)
{
	const FSlateBrush* Icon = FBlueprintEditorUtils::GetIconFromPin(InPinType, /*bIsLarge*/true);
	const FLinearColor IconColor = Schema->GetPinTypeColor(InPinType);

	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpacer)
			.Size(FVector2D(18.0f, 0.0f))
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(1.0f, 0.0f)
		[
			SNew(SImage)
			.Image(Icon)
			.ColorAndOpacity(IconColor)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(InDisplayText)
		];
}

FCachedBindingData::FCachedBindingData(IPropertyBindingBindingCollectionOwner* InPropertyBindingsOwner
	, const FPropertyBindingPath& InTargetPath
	, const TSharedPtr<const IPropertyHandle>& InPropertyHandle
	, const TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> InAccessibleStructs
	)
	: WeakBindingsOwner(InPropertyBindingsOwner)
	, TargetPath(InTargetPath)
	, PropertyHandle(InPropertyHandle)
	, AccessibleStructs(InAccessibleStructs)
{
}

void FCachedBindingData::AddBinding(TConstArrayView<FBindingChainElement> InBindingChain)
{
	if (InBindingChain.IsEmpty())
	{
		return;
	}
	
	if (!TargetPath.GetStructID().IsValid())
	{
		return;
	}

	IPropertyBindingBindingCollectionOwner* BindingOwner = WeakBindingsOwner.Get();
	if (BindingOwner == nullptr)
	{
		return;
	}

	FPropertyBindingBindingCollection* BindingsCollection = BindingOwner->GetEditorPropertyBindings();
	if (BindingsCollection == nullptr)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("BindingData_AddBinding", "Add Binding"));

	// First item in the binding chain is the index in AccessibleStructs.
	const int32 SourceStructIndex = InBindingChain[0].ArrayIndex;
	TConstStructView<FPropertyBindingBindableStructDescriptor> BindableStruct = GetBindableStructDescriptor(SourceStructIndex);
	const FGuid StructID = BindableStruct.Get().ID;

 	// Remove struct index.
	const TConstArrayView<FBindingChainElement> SourceBindingChain(InBindingChain.begin() + 1, InBindingChain.Num() - 1);

	// GetBindingDataViewByID can fail but we still need to call AddBindingInternal
	FPropertyBindingDataView DataView;
	BindingOwner->GetBindingDataViewByID(StructID, DataView);

	// If SourceBindingChain is empty at this stage, it means that the binding points to the source struct itself.
	FPropertyBindingPath SourcePath;
	MakeStructPropertyPathFromBindingChain(StructID, SourceBindingChain, DataView, SourcePath);

	UObject* BindingOwnerObject = WeakBindingsOwner.GetObject();
	BindingOwnerObject->Modify();

	// Allow derived classes to handle the bindings
	const bool bBindingHandled = AddBindingInternal(BindableStruct, SourcePath, TargetPath);
	if (bBindingHandled == false)
	{
		BindingsCollection->AddBinding(SourcePath, TargetPath);
	}

	UpdateData();

	BindingOwner->OnPropertyBindingChanged(SourcePath, TargetPath);
}

bool FCachedBindingData::HasBinding(const FPropertyBindingBindingCollection::ESearchMode SearchMode) const
{
	IPropertyBindingBindingCollectionOwner* BindingOwner = WeakBindingsOwner.Get();
	if (BindingOwner == nullptr)
	{
		return false;
	}

	const FPropertyBindingBindingCollection* EditorBindings = BindingOwner->GetEditorPropertyBindings();
	if (EditorBindings == nullptr)
	{
		return false;
	}

	return EditorBindings->HasBinding(TargetPath, SearchMode);
}

const FPropertyBindingBinding* FCachedBindingData::FindBinding(FPropertyBindingBindingCollection::ESearchMode SearchMode) const
{
	IPropertyBindingBindingCollectionOwner* BindingOwner = WeakBindingsOwner.Get();
	if (BindingOwner == nullptr)
	{
		return nullptr;
	}

	const FPropertyBindingBindingCollection* EditorBindings = BindingOwner->GetEditorPropertyBindings();
	if (EditorBindings == nullptr)
	{
		return nullptr;
	}

	return EditorBindings->FindBinding(TargetPath, SearchMode);
}

void FCachedBindingData::RemoveBinding(const FPropertyBindingBindingCollection::ESearchMode RemoveMode)
{
	IPropertyBindingBindingCollectionOwner* BindingOwner = WeakBindingsOwner.Get();
	if (BindingOwner == nullptr)
	{
		return;
	}

	FPropertyBindingBindingCollection* EditorBindings = BindingOwner->GetEditorPropertyBindings();
	if (EditorBindings == nullptr)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("BindingData_RemoveBinding", "Remove Binding"));
	UObject* OwnerObject = WeakBindingsOwner.GetObject();
	OwnerObject->Modify();
	EditorBindings->RemoveBindings(TargetPath, RemoveMode);

	UpdateData();
	
	const FPropertyBindingPath SourcePath; // Null path
	BindingOwner->OnPropertyBindingChanged(SourcePath, TargetPath);
}

// deprecated
bool FCachedBindingData::GetPinTypeAndIconForProperty(const FProperty& InProperty, FPropertyBindingDataView InTargetDataView
	, FEdGraphPinType& OutPinType, FName& OutIconName) const
{
	return false;
}

// deprecated
bool FCachedBindingData::GetPinTypeAndIconForProperty(const FProperty& InProperty, FPropertyBindingDataView InTargetDataView
	, FEdGraphPinType& OutPinType, const FSlateBrush*& OutIconBrush) const
{
	return false;
}

FCachedBindingData::FPinInfo FCachedBindingData::GetPinInfoForProperty(const TNotNull<const FProperty*> InProperty, FPropertyBindingDataView InTargetDataView) const
{
	FPinInfo Info;
	const UEdGraphSchema_K2* Schema = GetEdGraphSchema();
	Schema->ConvertPropertyToPinType(InProperty, Info.Type);
	Info.IconBrush = FAppStyle::GetBrush("Kismet.Tabs.Variables");
	Info.IconColor = Schema->GetPinTypeColor(Info.Type);
	return Info;
}

bool FCachedBindingData::IsPropertyReference(const FProperty& InProperty)
{
	return false;
}

void FCachedBindingData::UpdateSourcePropertyPath(TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, const FPropertyBindingPath& InSourcePath, FString& OutString)
{
}

void FCachedBindingData::UpdatePropertyReferenceTooltip(const FProperty& InProperty, FTextBuilder& InOutTextBuilder) const
{
}

void FCachedBindingData::UpdateData()
{
	SourceStructName = FText::GetEmpty();
	FormatableText = FText::GetEmpty();
	FormatableTooltipText = FText::GetEmpty();
	Color = FLinearColor::White;
	Image = nullptr;
	bIsCachedDataValid = false;

	if (!PropertyHandle.IsValid())
	{
		return;
	}

	const FProperty* Property = PropertyHandle->GetProperty();
	if (Property == nullptr)
	{
		return;
	}

	IPropertyBindingBindingCollectionOwner* BindingOwner = WeakBindingsOwner.Get();
	if (BindingOwner == nullptr)
	{
		return;
	}

	FPropertyBindingBindingCollection* EditorBindings = BindingOwner->GetEditorPropertyBindings();
	if (EditorBindings == nullptr)
	{
		return;
	}

	const UEdGraphSchema_K2* Schema = GetEdGraphSchema();
	check(Schema);

	FPropertyBindingDataView TargetDataView;
	BindingOwner->GetBindingDataViewByID(TargetPath.GetStructID(), TargetDataView);

	const bool bIsPropertyReference = IsPropertyReference(*Property);
	FPinInfo PinInfo = GetPinInfoForProperty(Property, TargetDataView);

	FTextBuilder TooltipBuilder;

	if (const FPropertyBindingBinding* CurrentBinding = EditorBindings->FindBinding(TargetPath))
	{
		const FPropertyBindingPath& SourcePath = CurrentBinding->GetSourcePath();
		FString SourcePropertyPathAsString = SourcePath.ToString();

		// If source is a bound PropertyFunction, it will not be present in AccessibleStructs thus it has to be accessed through bindings owner.
		TInstancedStruct<FPropertyBindingBindableStructDescriptor> SourceDesc;
		if (BindingOwner->GetBindableStructByID(SourcePath.GetStructID(), SourceDesc))
		{
			// Allow derived class to provide a different source path
			UpdateSourcePropertyPath(SourceDesc, SourcePath, SourcePropertyPathAsString);

			// Check that the binding is valid.
			bool bIsValidBinding = false;
			FPropertyBindingDataView SourceDataView;
			const FProperty* SourceLeafProperty = nullptr;
			const UStruct* SourceStruct = nullptr;
			if (BindingOwner->GetBindingDataViewByID(SourcePath.GetStructID(), SourceDataView)
				&& TargetDataView.IsValid())
			{
				TArray<FPropertyBindingPathIndirection> SourceIndirections;
				TArray<FPropertyBindingPathIndirection> TargetIndirections;

				// Resolve source and target properties.
				// Source path can be empty, when the binding binds directly to a context struct/class.
				// Target path must always point to a valid property (at least one indirection).
				if (SourcePath.ResolveIndirectionsWithValue(SourceDataView, SourceIndirections, /*error*/nullptr, /*bHandleRedirects*/true)
					&& TargetPath.ResolveIndirectionsWithValue(TargetDataView, TargetIndirections, /*error*/nullptr, /*bHandleRedirects*/true)
					&& !TargetIndirections.IsEmpty())
				{
					const FPropertyBindingPathIndirection TargetLeafIndirection = TargetIndirections.Last();
					if (SourceIndirections.Num() > 0)
					{
						// Binding to a source property.
						const FPropertyBindingPathIndirection SourceLeafIndirection = SourceIndirections.Last();
						SourceLeafProperty = SourceLeafIndirection.GetProperty();
						bIsValidBinding = ArePropertiesCompatible(SourceLeafProperty, TargetLeafIndirection.GetProperty(), SourceLeafIndirection.GetPropertyAddress(), TargetLeafIndirection.GetPropertyAddress());
					}
					else
					{
						// Binding to a source context struct.
						SourceStruct = SourceDataView.GetStruct();
						bIsValidBinding = ArePropertyAndContextStructCompatible(SourceStruct, TargetLeafIndirection.GetProperty());
					}
				}
			}

			FormatableText = FText::FormatNamed(LOCTEXT("ValidSourcePath", "{SourceStruct}{PropertyPath}")
				, TEXT("PropertyPath")
				, SourcePropertyPathAsString.IsEmpty() ? FText() : FText::FromString(TEXT(".") + SourcePropertyPathAsString));
			SourceStructName = FText::FromString(SourceDesc.Get().Name.ToString());

			if (bIsValidBinding)
			{
				if (SourcePropertyPathAsString.IsEmpty())
				{
					if (CurrentBinding->GetPropertyFunctionNode().IsValid())
					{
						TooltipBuilder.AppendLine(LOCTEXT("ExistingBindingToFunctionTooltip", "Property is bound to function {SourceStruct}."));
					}
					else
					{
						TooltipBuilder.AppendLine(LOCTEXT("ExistingBindingTooltip", "Property is bound to {SourceStruct}."));
					}
				}
				else
				{
					if (CurrentBinding->GetPropertyFunctionNode().IsValid())
					{
						TooltipBuilder.AppendLineFormat(LOCTEXT("ExistingBindingToFunctionWithPropertyTooltip", "Property is bound to function {SourceStruct} property {PropertyPath}."), { {TEXT("PropertyPath"), FText::FromString(SourcePropertyPathAsString)} });
					}
					else
					{
						TooltipBuilder.AppendLineFormat(LOCTEXT("ExistingBindingWithPropertyTooltip", "Property is bound to {SourceStruct} property {PropertyPath}."), { {TEXT("PropertyPath"), FText::FromString(SourcePropertyPathAsString)} });
					}
				}

				// Update pin type with source property so property reference that can bind
				// to multiple types display the bound one.
				if (bIsPropertyReference && SourceLeafProperty)
				{
					Schema->ConvertPropertyToPinType(SourceLeafProperty, PinInfo.Type);
					PinInfo.IconColor = Schema->GetPinTypeColor(PinInfo.Type);
				}

				Image = PinInfo.IconBrush;
				Color = PinInfo.IconColor;

				bIsCachedDataValid = true;
			}
			else
			{
				FText SourceType;
				if (SourceLeafProperty)
				{
					SourceType = GetPropertyTypeText(Schema, SourceLeafProperty);
				}
				else if (SourceStruct)
				{
					SourceType = SourceStruct->GetDisplayNameText();
				}
				FText TargetType = GetPropertyTypeText(Schema, Property);

				if (SourcePath.IsPathEmpty())
				{
					TooltipBuilder.AppendLineFormat(LOCTEXT("MismatchingBindingTooltip", "Property is bound to {SourceStruct}, but binding source type '{SourceType}' does not match property type '{TargetType}'."),
						{
							{TEXT("SourceType"), SourceType},
							{TEXT("TargetType"), TargetType}
						});
				}
				else
				{
					TooltipBuilder.AppendLineFormat(LOCTEXT("MismatchingBindingTooltipWithProperty", "Property is bound to {SourceStruct} property {PropertyPath}, but binding source type '{SourceType}' does not match property type '{TargetType}'."),
						{
							{TEXT("PropertyPath"), FText::FromString(SourcePropertyPathAsString)},
							{TEXT("SourceType"), SourceType},
							{TEXT("TargetType"), TargetType}
						});
				}

				Image = FCoreStyle::Get().GetBrush("Icons.ErrorWithColor");
				Color = FLinearColor::White;
			}
		}
		else
		{
			// Missing source
			FormatableText = FText::Format(LOCTEXT("MissingSource", "???.{0}"), FText::FromString(SourcePropertyPathAsString));
			TooltipBuilder.AppendLineFormat(LOCTEXT("MissingBindingTooltip", "Missing binding source for property path '{0}'."), FText::FromString(SourcePropertyPathAsString));
			Image = FCoreStyle::Get().GetBrush("Icons.ErrorWithColor");
			Color = FLinearColor::White;
		}

		CachedSourcePath = SourcePath;
	}
	else
	{
		// No bindings
		FormatableText = FText::GetEmpty();
		TooltipBuilder.AppendLineFormat(LOCTEXT("BindTooltip", "Bind {0} to value from another property."), UE::PropertyBinding::GetPropertyTypeText(GetEdGraphSchema(), Property));

		Image = PinInfo.IconBrush;
		Color = PinInfo.IconColor;

		bIsCachedDataValid = true;

		CachedSourcePath.Reset();
	}

	if (bIsPropertyReference)
	{
		UpdatePropertyReferenceTooltip(*Property, TooltipBuilder);
	}

	FormatableTooltipText = TooltipBuilder.ToText();

	bIsDataCached = true;
}

bool FCachedBindingData::CanBindToContextStruct(const UStruct* InStruct, int32 InStructIndex)
{
	ConditionallyUpdateData();

	return CanBindToContextStructInternal(InStruct, InStructIndex);
}

void FCachedBindingData::GetSourceDataViewForNewBinding(TNotNull<IPropertyBindingBindingCollectionOwner*> InBindingsOwner, TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor
	, FPropertyBindingDataView& OutSourceDataView)
{
	InBindingsOwner->GetBindingDataViewByID(InDescriptor.Get().ID, OutSourceDataView);
}

bool FCachedBindingData::CanBindToProperty(const FProperty* SourceProperty, TConstArrayView<FBindingChainElement> InBindingChain)
{
	ConditionallyUpdateData();

	// Special case for binding widget calling OnCanBindProperty with Args.Property (i.e. self).
	if (PropertyHandle->GetProperty() == SourceProperty)
	{
		return true;
	}

	IPropertyBindingBindingCollectionOwner* BindingOwner = WeakBindingsOwner.Get();
	if (BindingOwner == nullptr)
	{
		return false;
	}

	const int32 SourceStructIndex = InBindingChain[0].ArrayIndex;
	const TConstStructView BindableStruct = GetBindableStructDescriptor(SourceStructIndex);

	FPropertyBindingDataView SourceDataView;
	GetSourceDataViewForNewBinding(BindingOwner, BindableStruct, SourceDataView);

	FPropertyBindingPath SourcePath;
	MakeStructPropertyPathFromBindingChain(BindableStruct.Get().ID, InBindingChain, SourceDataView, SourcePath);

	TArray<FPropertyBindingPathIndirection> SourceIndirections;
	void* TargetValueAddress = nullptr;
	if (PropertyHandle->GetValueData(TargetValueAddress) == FPropertyAccess::Success && SourcePath.ResolveIndirectionsWithValue(SourceDataView, SourceIndirections))
	{
		return ArePropertiesCompatible(SourceProperty, PropertyHandle->GetProperty(), SourceIndirections.Last().GetPropertyAddress(), TargetValueAddress);
	}

	return false;
}

bool FCachedBindingData::CanAcceptPropertyOrChildrenInternal(const FProperty& InProperty, TConstArrayView<FBindingChainElement, int> InBindingChain)
{
	return true;
}

bool FCachedBindingData::AddBindingInternal(TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, FPropertyBindingPath& InOutSourcePath
	, const FPropertyBindingPath& InTargetPath)
{
	// We will use the default behavior
	return false;
}

bool FCachedBindingData::CanAcceptPropertyOrChildren(const FProperty* SourceProperty, const TConstArrayView<FBindingChainElement> InBindingChain)
{
	if (!SourceProperty)
	{
		return false;
	}

	ConditionallyUpdateData();

	if (!PropertyHandle.IsValid() || PropertyHandle->GetProperty() == nullptr)
	{
		return false;
	}

	if (CanAcceptPropertyOrChildrenInternal(*SourceProperty, InBindingChain))
	{
		return IsPropertyBindable(*SourceProperty);
	}

	return false;
}

void FCachedBindingData::PromoteToParameter(const FName InPropertyName, TConstStructView<FPropertyBindingBindableStructDescriptor> InStructDesc, const TSharedPtr<const FPropertyInfoOverride> InPropertyInfoOverride)
{
	if (!TargetPath.GetStructID().IsValid())
	{
		return;
	}

	IPropertyBindingBindingCollectionOwner* BindingsOwner = WeakBindingsOwner.Get();
	if (!BindingsOwner)
	{
		return;
	}

	const FProperty* Property = PropertyHandle->GetProperty();
	if (!Property)
	{
		return;
	}

	const FProperty* TargetProperty = nullptr;
	const void* TargetContainerAddress = nullptr;

	FPropertyBindingDataView TargetDataView;
	if (BindingsOwner->GetBindingDataViewByID(TargetPath.GetStructID(), TargetDataView) && TargetDataView.IsValid())
	{
		TArray<FPropertyBindingPathIndirection> TargetIndirections;
		FString Error;
		const bool bResolved = TargetPath.ResolveIndirectionsWithValue(TargetDataView, TargetIndirections, &Error);

		if (ensureMsgf(bResolved, TEXT("Failed to resolve indirections: %s"), *Error))
		{
			const FPropertyBindingPathIndirection& LastIndirection = TargetIndirections.Last();
			TargetProperty = LastIndirection.GetProperty();
			TargetContainerAddress = LastIndirection.GetContainerAddress();
		}
	}

	FPropertyBindingBindingCollection* EditorBindings = BindingsOwner->GetEditorPropertyBindings();
	if (!EditorBindings)
	{
		return;
	}

	const FGuid StructID = InStructDesc.Get().ID;

	const FScopedTransaction Transaction(LOCTEXT("PromoteToParameter", "Promote to Parameter"));

	TArray<FPropertyCreationDescriptor, TFixedAllocator<1>> PropertyCreationDescs;
	{
		FPropertyCreationDescriptor& PropertyCreationDesc = PropertyCreationDescs.AddDefaulted_GetRef();

		if (InPropertyInfoOverride)
		{
			PropertyCreationDesc.PropertyDesc.Name = InPropertyName;
			StructUtils::SetPropertyDescFromPin(PropertyCreationDesc.PropertyDesc, InPropertyInfoOverride->PinType);
		}
		else
		{
			PropertyCreationDesc.PropertyDesc = FPropertyBagPropertyDesc(InPropertyName, Property);
		}

		BindingsOwner->OnPromotingToParameter(PropertyCreationDesc.PropertyDesc);

		// Set the Property & Container Address to copy
		if (TargetProperty && TargetContainerAddress)
		{
			PropertyCreationDesc.SourceProperty = TargetProperty;
			PropertyCreationDesc.SourceContainerAddress = TargetContainerAddress;
		}
	}

	Cast<UObject>(BindingsOwner)->Modify();

	BindingsOwner->CreateParametersForStruct(StructID, /*InOut*/PropertyCreationDescs);

	// Use the name in PropertyDescs, as it might contain a different name than the desired InPropertyName (for uniqueness)
	const FPropertyBindingPath SourcePath(StructID, PropertyCreationDescs[0].PropertyDesc.Name);
	EditorBindings->AddBinding(SourcePath, TargetPath);

	UpdateData();

	BindingsOwner->OnPropertyBindingChanged(SourcePath, TargetPath);
}

void FCachedBindingData::AddPropertyInfoOverride(const FProperty& InProperty, TArray<TSharedPtr<const UE::PropertyBinding::FPropertyInfoOverride>>& OutPropertyInfoOverrides) const
{
}

bool FCachedBindingData::CanCreateParameter(const FPropertyBindingBindableStructDescriptor& InStructDesc, TArray<TSharedPtr<const FPropertyInfoOverride>>& OutPropertyInfoOverrides) const
{
	const FProperty* Property = PropertyHandle->GetProperty();
	if (Property == nullptr)
	{
		return false;
	}

	// Is the type supported by the property bag
	const FPropertyBagPropertyDesc Desc = FPropertyBagPropertyDesc(Property->GetFName(), Property);
	if (Desc.ValueType == EPropertyBagPropertyType::None)
	{
		return false;
	}

	IPropertyBindingBindingCollectionOwner* BindingsOwner = WeakBindingsOwner.Get();
	if (BindingsOwner == nullptr)
	{
		return false;
	}

	const FPropertyBindingBindingCollection* EditorBindings = BindingsOwner->GetEditorPropertyBindings();
	if (EditorBindings == nullptr)
	{
		return false;
	}

	// Allow classes implementing IPropertyBindingBindingCollectionOwner to block parameter creation for that struct
	if (!BindingsOwner->CanCreateParameter(InStructDesc.ID))
	{
		return false;
	}

	// Allow derived classes to push overrides
	AddPropertyInfoOverride(*Property, OutPropertyInfoOverrides);

	return true;
}

bool FCachedBindingData::ArePropertyAndContextStructCompatible(const UStruct* SourceStruct, const FProperty* TargetProperty) const
{
	return DeterminePropertyAndContextStructCompatibleInternal(SourceStruct, TargetProperty);
}

bool FCachedBindingData::DeterminePropertyAndContextStructCompatibleInternal(const UStruct* SourceStruct, const FProperty* TargetProperty) const
{
	if (const FStructProperty* TargetStructProperty = CastField<FStructProperty>(TargetProperty))
	{
		return TargetStructProperty->Struct == SourceStruct;
	}
	if (const FObjectProperty* TargetObjectProperty = CastField<FObjectProperty>(TargetProperty))
	{
		return SourceStruct != nullptr && SourceStruct->IsChildOf(TargetObjectProperty->PropertyClass);
	}
	
	return false;
}

bool FCachedBindingData::ArePropertiesCompatible(const FProperty* SourceProperty, const FProperty* TargetProperty, const void* SourcePropertyValue, const void* TargetPropertyValue) const
{
	bool bAreCompatible = false;
	if (DeterminePropertiesCompatibilityInternal(SourceProperty, TargetProperty, SourcePropertyValue, TargetPropertyValue, bAreCompatible))
	{
		return bAreCompatible;
	}

	// Note: We support type promotion here
	return GetPropertyCompatibility(SourceProperty, TargetProperty) != EPropertyCompatibility::Incompatible;
}

UStruct* FCachedBindingData::ResolveIndirection(TConstArrayView<FBindingChainElement> InBindingChain)
{
	const IPropertyBindingBindingCollectionOwner* PropertyBindingsOwner = WeakBindingsOwner.Get();
	if (!PropertyBindingsOwner)
	{
		return nullptr;
	}

	const int32 SourceStructIndex = InBindingChain[0].ArrayIndex;
	
	TArray<FBindingChainElement> SourceBindingChain(InBindingChain);
	SourceBindingChain.RemoveAt(0);

	FPropertyBindingDataView DataView;
	if (PropertyBindingsOwner->GetBindingDataViewByID(GetBindableStructDescriptor(SourceStructIndex).Get().ID, DataView))
	{
		return ResolveLeafValueStructType(DataView, InBindingChain);
	}

	return nullptr;
}

bool FCachedBindingData::CanBindToContextStructInternal(const UStruct* InStruct, int32 InStructIndex)
{
	return ArePropertyAndContextStructCompatible(InStruct, PropertyHandle->GetProperty());
}

bool FCachedBindingData::GetPropertyFunctionText(FConstStructView InPropertyFunctionStructView, FText& OutText) const
{
	return false;
}

bool FCachedBindingData::GetPropertyFunctionTooltipText(FConstStructView InPropertyFunctionStructView, FText& OutText) const
{
	return false;
}

bool FCachedBindingData::GetPropertyFunctionIconColor(FConstStructView InPropertyFunctionStructView, FLinearColor& OutColor) const
{
	return false;
}

bool FCachedBindingData::GetPropertyFunctionImage(FConstStructView InPropertyFunctionStructView, const FSlateBrush*& OutImage) const
{
	return false;
}

bool FCachedBindingData::DeterminePropertiesCompatibilityInternal(const FProperty* InSourceProperty, const FProperty* InTargetProperty
	, const void* InSourcePropertyValue, const void* InTargetPropertyValue, bool& bOutAreCompatible) const
{
	return false;
}

bool ExecuteOnFunctionStructView(IPropertyBindingBindingCollectionOwner* InBindingOwner, const FPropertyBindingPath& InTargetPath, TFunctionRef<bool(FConstStructView StructView)> InFunction)
{
	if (InBindingOwner)
	{
		if (const FPropertyBindingBindingCollection* EditorBindings = InBindingOwner->GetEditorPropertyBindings())
		{
			if (const FPropertyBindingBinding* CurrentBinding = EditorBindings->FindBinding(InTargetPath, FPropertyBindingBindingCollection::ESearchMode::Exact))
			{
				const FConstStructView PropertyFunctionStructView = CurrentBinding->GetPropertyFunctionNode();
				if (PropertyFunctionStructView.IsValid())
				{
					return InFunction(PropertyFunctionStructView);
				}
			}
		}
	}

	return false;
}

FText FCachedBindingData::GetText()
{
	ConditionallyUpdateData();

	if (bIsCachedDataValid)
	{
		// Bound PropertyFunction is allowed to override its display name.
		FText CustomText;
		if (ExecuteOnFunctionStructView(WeakBindingsOwner.Get(), TargetPath, [&CustomText, this](FConstStructView StructView)
			{
				return GetPropertyFunctionText(StructView, CustomText);
			}))
		{
			return CustomText;
		}
	}

	return FText::FormatNamed(FormatableText, TEXT("SourceStruct"), SourceStructName);
}

FText FCachedBindingData::GetTooltipText()
{
	ConditionallyUpdateData();

	if (bIsCachedDataValid)
	{
		// If the source property is a PropertyFunction and it overrides its display name, it's been used in the tooltip text.
		FText CustomText;
		if (ExecuteOnFunctionStructView(WeakBindingsOwner.Get(), TargetPath, [&CustomText, this](FConstStructView StructView)
			{
				return GetPropertyFunctionTooltipText(StructView, CustomText);
			}))
		{
			return CustomText;
		}
	}

	return FText::FormatNamed(FormatableTooltipText, TEXT("SourceStruct"), SourceStructName);
}

FLinearColor FCachedBindingData::GetColor()
{
	ConditionallyUpdateData();

	if (bIsCachedDataValid)
	{
		// Bound PropertyFunction is allowed to override its icon color if the binding leads directly into it's single output property.
		if (CachedSourcePath.NumSegments() == 1)
		{
			FLinearColor CustomColor;
			if (ExecuteOnFunctionStructView(WeakBindingsOwner.Get(), TargetPath, [&CustomColor, this](FConstStructView StructView)
				{
					return GetPropertyFunctionIconColor(StructView, CustomColor);
				}))
			{
				return CustomColor;
			}
		}
	}

	return Color;
}

const FSlateBrush* FCachedBindingData::GetImage()
{
	ConditionallyUpdateData();

	if (bIsCachedDataValid)
	{
		// Bound PropertyFunction is allowed to override its icon.
		const FSlateBrush* CustomImage;
		if (ExecuteOnFunctionStructView(WeakBindingsOwner.Get(), TargetPath, [&CustomImage, this](FConstStructView StructView)
			{
				return GetPropertyFunctionImage(StructView, CustomImage);
			}))
		{
			return CustomImage;
		}
	}

	return Image;
}


TNotNull<const UEdGraphSchema_K2*> FCachedBindingData::GetEdGraphSchema() const
{
	return GetDefault<UEdGraphSchema_K2>();
}

void FCachedBindingData::ConditionallyUpdateData()
{
	IPropertyBindingBindingCollectionOwner* BindingOwner = WeakBindingsOwner.Get();
	if (BindingOwner == nullptr)
	{
		return;
	}

	const FPropertyBindingBindingCollection* EditorBindings = BindingOwner->GetEditorPropertyBindings();
	if (EditorBindings == nullptr)
	{
		return;
	}

	const FPropertyBindingPath* CurrentSourcePath = EditorBindings->GetBindingSource(TargetPath);
	bool bPathsIdentical = false;
	if (CurrentSourcePath)
	{
		bPathsIdentical = CachedSourcePath == *CurrentSourcePath;
	}
	else
	{
		bPathsIdentical = CachedSourcePath.IsPathEmpty();
	}

	if (!bIsDataCached || !bPathsIdentical)
	{
		UpdateData();
	}
}

bool IsPropertyBindable(const FProperty& Property)
{
	const bool bIsUserEditable = Property.HasAnyPropertyFlags(CPF_Edit);
	if (!bIsUserEditable)
	{
		UE_LOGF(LogPropertyBindingUtils, Verbose, "Property %ls is not bindable because it's not user-settable in the editor",
			*Property.GetName());
		return false;
	}

	const bool bPrivateOrProtected = !Property.HasAnyPropertyFlags(CPF_NativeAccessSpecifierPrivate | CPF_NativeAccessSpecifierProtected);
	const bool bPrivateButBlueprintAccessible = Property.GetBoolMetaData(FBlueprintMetadata::MD_AllowPrivateAccess);
	if (!bPrivateOrProtected && !bPrivateButBlueprintAccessible)
	{
		UE_LOGF(LogPropertyBindingUtils, Verbose, "Property %ls is not bindable because it's either private or protected and not private-accessible to blueprints",
			*Property.GetName());
		return false;
	}

	return true;
}

} // UE::PropertyBinding

bool FPropertyBindingExtension::IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const
{
	const FProperty* Property = InPropertyHandle.GetProperty();
	if (Property == nullptr || Property->HasAnyPropertyFlags(GetDisallowedPropertyFlags()))
	{
		return false;
	}

	// Does the container or property support bindings
	if (UE::PropertyBinding::HasMetaData(UE::PropertyBinding::MetaDataNoBindingName, InPropertyHandle.AsShared()))
	{
		return false;
	}

	TArray<UObject*> OuterObjects;
	InPropertyHandle.GetOuterObjects(OuterObjects);
	if (OuterObjects.Num() == 1)
	{
		// Only allow to bind when one object is selected.
		if (const IPropertyBindingBindingCollectionOwner* BindingsOwner =  UE::PropertyBinding::FindBindingsOwner(OuterObjects[0]))
		{
			FPropertyBindingPath TargetPath;

			// Figure out the structs we're editing, and property path relative to current property.
			const FGuid FallbackStructID = BindingsOwner->GetFallbackStructID();
			const TSharedPtr<const IPropertyHandle> SharedHandle = UE::PropertyBinding::MakeStructPropertyPathFromPropertyHandle(
				InPropertyHandle.AsShared(), TargetPath, FallbackStructID);

			if (!TargetPath.GetStructID().IsValid())
			{
				return false;
			}
			else if (const IPropertyHandle* BindablePropertyHandle = SharedHandle.Get())
			{
				if (!CanBindToProperty(TargetPath, *BindablePropertyHandle))
				{
					return false;
				}
			}
		}
	}
	return true;
}

void FPropertyBindingExtension::ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	if (!IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
	{
		return;
	}

	using namespace UE::PropertyBinding;

	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
	IPropertyBindingBindingCollectionOwner* BindingsOwner = nullptr;

	// Array of structs we can bind to.
	TArray<FBindingContextStruct> BindingContextStructs;
	TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> AccessibleStructs;

	// The struct and property where we're binding.
	FPropertyBindingPath TargetPath;

	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num() == 1)
	{
		// Only allow to bind when one object is selected.
		BindingsOwner = FindBindingsOwner(OuterObjects[0]);
		if (BindingsOwner)
		{
			const FGuid FallbackStructID = BindingsOwner->GetFallbackStructID();
			// Figure out the structs we're editing, and property path relative to current property.
			MakeStructPropertyPathFromPropertyHandle(InPropertyHandle, TargetPath, FallbackStructID);

			BindingsOwner->GetBindableStructs(TargetPath.GetStructID(), AccessibleStructs);
			BindingsOwner->AppendBindablePropertyFunctionStructs(AccessibleStructs);

			TMap<FString, FText> SectionNames;
			for (TInstancedStruct InstancedStruct  : AccessibleStructs)
			{
				const FPropertyBindingBindableStructDescriptor& StructDesc = InstancedStruct.Get();
				const UStruct* Struct = StructDesc.Struct;

				FBindingContextStruct& ContextStruct = BindingContextStructs.AddDefaulted_GetRef();
				ContextStruct.DisplayText = FText::FromString(StructDesc.Name.ToString());
				ContextStruct.Struct = const_cast<UStruct*>(Struct);
				ContextStruct.Category = StructDesc.Category;

				UpdateContextStruct(InstancedStruct, ContextStruct, SectionNames);
			}
		}
	}

	TSharedPtr<FCachedBindingData> CachedBindingData = CreateCachedBindingData(BindingsOwner, TargetPath, InPropertyHandle, AccessibleStructs);

	FPropertyBindingWidgetArgs Args;
	Args.Property = InPropertyHandle->GetProperty();

	Args.OnCanBindPropertyWithBindingChain = FOnCanBindPropertyWithBindingChain::CreateLambda([CachedBindingData](FProperty* InProperty, TConstArrayView<FBindingChainElement> InBindingChain)
		{
			return CachedBindingData->CanBindToProperty(InProperty, InBindingChain);
		});

	Args.OnCanBindToContextStructWithIndex = FOnCanBindToContextStructWithIndex::CreateLambda([CachedBindingData](const UStruct* InStruct, int32 InStructIndex)
		{
			return CachedBindingData->CanBindToContextStruct(InStruct, InStructIndex);
		});

	Args.OnCanAcceptPropertyOrChildrenWithBindingChain = FOnCanAcceptPropertyOrChildrenWithBindingChain::CreateLambda([CachedBindingData](FProperty* InProperty, TConstArrayView<FBindingChainElement> InBindingChain)
		{
			return CachedBindingData->CanAcceptPropertyOrChildren(InProperty, InBindingChain);
		});

	Args.OnCanBindToClass = FOnCanBindToClass::CreateLambda([](UClass* InClass)
		{
			return true;
		});

	Args.OnAddBinding = FOnAddBinding::CreateLambda([CachedBindingData, &InDetailBuilder](FName InPropertyName, TConstArrayView<FBindingChainElement> InBindingChain)
		{
			CachedBindingData->AddBinding(InBindingChain);
			InDetailBuilder.GetPropertyUtilities()->RequestForceRefresh();
		});

	Args.OnRemoveBinding = FOnRemoveBinding::CreateLambda([CachedBindingData, &InDetailBuilder](FName InPropertyName)
		{
			CachedBindingData->RemoveBinding(FPropertyBindingBindingCollection::ESearchMode::Exact);
			InDetailBuilder.GetPropertyUtilities()->RequestForceRefresh();
		});

	Args.OnCanRemoveBinding = FOnCanRemoveBinding::CreateLambda([CachedBindingData](FName InPropertyName)
		{
			return CachedBindingData->HasBinding(FPropertyBindingBindingCollection::ESearchMode::Exact);
		});

	Args.CurrentBindingText = MakeAttributeLambda([CachedBindingData]()
		{
			return CachedBindingData->GetText();
		});

	Args.CurrentBindingToolTipText = MakeAttributeLambda([CachedBindingData]()
		{
			return CachedBindingData->GetTooltipText();
		});

	Args.CurrentBindingImage = MakeAttributeLambda([CachedBindingData]() -> const FSlateBrush*
		{
			return CachedBindingData->GetImage();
		});

	Args.CurrentBindingColor = MakeAttributeLambda([CachedBindingData]() -> FLinearColor
		{
			return CachedBindingData->GetColor();
		});

	if (BindingsOwner)
	{
		Args.OnResolveIndirection = FOnResolveIndirection::CreateLambda([CachedBindingData](TConstArrayView<FBindingChainElement> InBindingChain)
		{
			return CachedBindingData->ResolveIndirection(InBindingChain);
		});
	}

	Args.BindButtonStyle = &FAppStyle::Get().GetWidgetStyle<FButtonStyle>("HoverHintOnly");
	Args.bAllowNewBindings = false;
	Args.bAllowArrayElementBindings = CanBindToArrayElements();
	Args.bAllowUObjectFunctions = false;

	if (CanPromoteToParameter(InPropertyHandle))
	{
		auto PromoteToParameterAndRefresh = [CachedBindingData, &InDetailBuilder](FName InPropertyName, TConstStructView<FPropertyBindingBindableStructDescriptor> InStructDesc, TSharedPtr<const FPropertyInfoOverride> InPropertyInfoOverride)
			{
				CachedBindingData->PromoteToParameter(InPropertyName, InStructDesc, InPropertyInfoOverride);
				InDetailBuilder.GetPropertyUtilities()->RequestForceRefresh();
			};

		Args.MenuExtender = MakeShared<FExtender>();
		Args.MenuExtender->AddMenuExtension(
			TEXT("BindingActions"),
			EExtensionHook::After,
			/*CommandList*/ nullptr,
			FMenuExtensionDelegate::CreateLambda([CachedBindingData, AccessibleStructs = MoveTemp(AccessibleStructs), InPropertyHandle, PromoteToParameterAndRefresh](FMenuBuilder& MenuBuilder)
			{
				const FProperty* Property = InPropertyHandle->GetProperty();
				check(Property);
				const FName PropertyName = Property->GetFName();
				const TSharedRef<FCachedBindingData> CachedBindingDataRef = CachedBindingData.ToSharedRef();

				auto AddMenuEntry = [PropertyName, PromoteToParameterAndRefresh](FMenuBuilder& InMenuBuilder, const FPropertyBindingBindableStructDescriptor& ContextStruct, const TSharedRef<FCachedBindingData>& InCachedBindingData, FMenuSectionHelper& InSectionHelper)
					{
						TArray<TSharedPtr<const FPropertyInfoOverride>> PropertyInfoOverrides;
						if (InCachedBindingData->CanCreateParameter(ContextStruct, /*out*/PropertyInfoOverrides))
						{
							const FString Section = ContextStruct.GetSection();
							if (!Section.IsEmpty())
							{
								InSectionHelper.SetSection(FText::FromString(Section));
							}

							if (PropertyInfoOverrides.IsEmpty())
							{
								InMenuBuilder.AddMenuEntry(FExecuteAction::CreateLambda(
									[PromoteToParameterAndRefresh, PropertyName, ContextStruct]()
										{
											PromoteToParameterAndRefresh(PropertyName, TConstStructView<FPropertyBindingBindableStructDescriptor>(ContextStruct), nullptr);
										}),
									MakeContextStructWidget(InCachedBindingData->GetEdGraphSchema(), ContextStruct));
							}
							else
							{
								InMenuBuilder.AddSubMenu(MakeContextStructWidget(InCachedBindingData->GetEdGraphSchema(), ContextStruct),
									FNewMenuDelegate::CreateLambda([InCachedBindingData, PropertyName, ContextStruct, PropertyInfoOverrides = MoveTemp(PropertyInfoOverrides), PromoteToParameterAndRefresh](FMenuBuilder& InSubMenuBuilder)
										{
											FMenuSectionHelper SectionHelper(InSubMenuBuilder);
											SectionHelper.SetSection(LOCTEXT("RefTypeParams", "Reference Types"));
											for (const TSharedPtr<const FPropertyInfoOverride>& PropertyInfoOverride : PropertyInfoOverrides)
											{
												InSubMenuBuilder.AddMenuEntry(
													FExecuteAction::CreateLambda(
														[PromoteToParameterAndRefresh, PropertyName, &ContextStruct, PropertyInfoOverride]()
														{
															PromoteToParameterAndRefresh(PropertyName, TConstStructView<FPropertyBindingBindableStructDescriptor>(ContextStruct), PropertyInfoOverride);
														}),
													MakeBindingPropertyInfoWidget(InCachedBindingData->GetEdGraphSchema(), PropertyInfoOverride->TypeNameText, PropertyInfoOverride->PinType));
											}
										}));
							}
						}
					};

				if (AccessibleStructs.Num() > 1)
				{
					MenuBuilder.AddSubMenu(
						LOCTEXT("PromoteToParameter", "Promote to Parameter"),
						LOCTEXT("PromoteToParameterTooltip", "Create a new parameter of the same type as the property, copy value over, and bind the property to the new parameter."),
						FNewMenuDelegate::CreateLambda([CachedBindingDataRef, &AccessibleStructs, &InPropertyHandle, PropertyName, AddMenuEntry](FMenuBuilder& InMenuBuilder)
							{
								FMenuSectionHelper SectionHelper(InMenuBuilder);
								for (const TInstancedStruct<FPropertyBindingBindableStructDescriptor>& InstancedContextStruct : AccessibleStructs)
								{
									AddMenuEntry(InMenuBuilder, InstancedContextStruct.Get(), CachedBindingDataRef, SectionHelper);
								}
							})
					);
				}
				else if (AccessibleStructs.Num() == 1)
				{
					TArray<TSharedPtr<const FPropertyInfoOverride>> PropertyInfoOverrides;
					if (CachedBindingDataRef->CanCreateParameter(AccessibleStructs[0].Get(), /*out*/PropertyInfoOverrides))
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("PromoteToParameter", "Promote to Parameter"),
							LOCTEXT("PromoteToParameterTooltip", "Create a new parameter of the same type as the property, copy value over, and bind the property to the new parameter."),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
							FUIAction(FExecuteAction::CreateLambda(
									[PromoteToParameterAndRefresh, PropertyName, AccessibleStruct = AccessibleStructs[0]]()
									{
										PromoteToParameterAndRefresh(PropertyName, TConstStructView<FPropertyBindingBindableStructDescriptor>(AccessibleStruct), TSharedPtr<const FPropertyInfoOverride>());
									})));
					}
				}
			})
		);
	}

	CustomizeDetailWidgetRow(InWidgetRow, InDetailBuilder, BindingsOwner, InPropertyHandle, TargetPath, CachedBindingData);

	CustomizePropertyBindingWidgetArgs(Args, BindingsOwner, TargetPath, CachedBindingData);

	InWidgetRow.ExtensionContent()
	[
		PropertyAccessEditor.MakePropertyBindingWidget(BindingContextStructs, Args)
	];
}

TSharedPtr<UE::PropertyBinding::FCachedBindingData> FPropertyBindingExtension::CreateCachedBindingData(IPropertyBindingBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath, const TSharedPtr<IPropertyHandle>& InPropertyHandle, const TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> InAccessibleStructs) const
{
	return MakeShared<UE::PropertyBinding::FCachedBindingData>(InBindingsOwner, InTargetPath, InPropertyHandle, InAccessibleStructs);
}

bool FPropertyBindingExtension::GetPromotionToParameterOverrideInternal(const FProperty& InProperty, bool& bOutOverride) const
{
	return false;
}

void FPropertyBindingExtension::UpdateContextStruct(TConstStructView<FPropertyBindingBindableStructDescriptor> InStructDesc, FBindingContextStruct& InOutContextStruct
	, TMap<FString, FText>& InOutSectionNames) const
{
	// nothing to do
}

void FPropertyBindingExtension::CustomizeDetailWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, IPropertyBindingBindingCollectionOwner* InBindingsOwner, TSharedPtr<IPropertyHandle> InPropertyHandle, const FPropertyBindingPath& InTargetPath, TSharedPtr<UE::PropertyBinding::FCachedBindingData> InCachedBindingData) const
{
	WrapValueWidget(InWidgetRow, InCachedBindingData);
	CustomizeResetToDefault(InWidgetRow, InDetailBuilder, InPropertyHandle, InCachedBindingData);
}

void FPropertyBindingExtension::WrapValueWidget(FDetailWidgetRow& InWidgetRow,
	const TSharedPtr<UE::PropertyBinding::FCachedBindingData>& InCachedBindingData) const
{
	auto IsValueVisible = TAttribute<EVisibility>::Create([InCachedBindingData]() -> EVisibility
	{
		return InCachedBindingData->HasBinding(FPropertyBindingBindingCollection::ESearchMode::Exact) ? EVisibility::Collapsed : EVisibility::Visible;
	});

	TSharedPtr<SWidget> ValueWidget = InWidgetRow.ValueContent().Widget;
	InWidgetRow.ValueContent()
	[
		SNew(SBox)
		.Visibility(IsValueVisible)
		[
			ValueWidget.ToSharedRef()
		]
	];
}

void FPropertyBindingExtension::CustomizeResetToDefault(FDetailWidgetRow& InWidgetRow,
	const IDetailLayoutBuilder& InDetailBuilder, const TSharedPtr<IPropertyHandle>& InPropertyHandle,
	const TSharedPtr<UE::PropertyBinding::FCachedBindingData>& InCachedBindingData) const
{
	InWidgetRow.CustomResetToDefault = FResetToDefaultOverride::Create(
		MakeAttributeLambda([InCachedBindingData, InPropertyHandle]()
			{
				return InPropertyHandle->CanResetToDefault() || InCachedBindingData->HasBinding(FPropertyBindingBindingCollection::ESearchMode::Includes);
			}),
		FSimpleDelegate::CreateLambda([InCachedBindingData, &InDetailBuilder, InPropertyHandle]()
			{
				if (InCachedBindingData->HasBinding(FPropertyBindingBindingCollection::ESearchMode::Includes))
				{
					InCachedBindingData->RemoveBinding(FPropertyBindingBindingCollection::ESearchMode::Includes);
					InDetailBuilder.GetPropertyUtilities()->RequestForceRefresh();
				}
				if (InPropertyHandle->CanResetToDefault())
				{
					InPropertyHandle->ResetToDefault();
				}
			}),
		false);
}

void FPropertyBindingExtension::CustomizePropertyBindingWidgetArgs(FPropertyBindingWidgetArgs& Args,
	IPropertyBindingBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath,
	TSharedPtr<UE::PropertyBinding::FCachedBindingData> InCachedBindingData)
{
	
}

bool FPropertyBindingExtension::CanBindToProperty(const FPropertyBindingPath& InTargetPath, const IPropertyHandle& InPropertyHandle) const
{
	return true;
}

uint64 FPropertyBindingExtension::GetDisallowedPropertyFlags() const
{
	return CPF_PersistentInstance | CPF_EditorOnly | CPF_Config | CPF_Deprecated;
}

bool FPropertyBindingExtension::CanPromoteToParameter(const TSharedPtr<IPropertyHandle>& InPropertyHandle) const
{
	const FProperty* Property = InPropertyHandle->GetProperty();
	if (Property == nullptr)
	{
		return false;
	}

	if (bool bCanPromoteOverride = false; GetPromotionToParameterOverrideInternal(*Property, bCanPromoteOverride))
	{
		return bCanPromoteOverride;
	}

	// Property Bag picker only detects Blueprint Types, so only allow properties that are blueprint types
	// FPropertyBagInstanceDataDetails::OnPropertyNameContent uses SPinTypeSelector to generate the property type picker.
	// UEdGraphSchema_K2::GetVariableTypeTree (GatherPinsImpl: FindEnums, FindStructs, FindObjectsAndInterfaces) is used there which only allows bp types.
	// The below behavior mirrors the behavior in the pin gathering but for properties

	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		if (!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(EnumProperty->GetEnum()))
		{
			return false;
		}
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		if (!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(StructProperty->Struct, /*bForInternalUse*/true))
		{
			return false;
		}
	}
	else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
	{
		if (!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(ObjectProperty->PropertyClass))
		{
			return false;
		}
	}
	else if (const FInterfaceProperty* InterfaceProperty = CastField<FInterfaceProperty>(Property))
	{
		if (!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(InterfaceProperty->InterfaceClass))
		{
			return false;
		}
	}

	// Is the type supported by the property bag
	const FPropertyBagPropertyDesc Desc = FPropertyBagPropertyDesc(Property->GetFName(), Property);
	if (Desc.ValueType == EPropertyBagPropertyType::None)
	{
		return false;
	}

	if (UE::PropertyBinding::HasMetaData(UE::PropertyBinding::MetaDataNoPromoteToParameter, InPropertyHandle.ToSharedRef()))
	{
		return false;
	}

	return true;
}

bool FPropertyBindingExtension::CanBindToArrayElements() const
{
	return false;
}

TNotNull<const UEdGraphSchema_K2*> FPropertyBindingExtension::GetEdGraphSchema() const
{
	return GetDefault<UEdGraphSchema_K2>();
}

// deprecated
TSharedRef<SWidget> FPropertyBindingExtension::MakeContextStructWidget(const FPropertyBindingBindableStructDescriptor& InContextStruct)
{
	return MakeContextStructWidget(GetDefault<UEdGraphSchema_K2>(), InContextStruct);
}

TSharedRef<SWidget> FPropertyBindingExtension::MakeContextStructWidget(TNotNull<const UEdGraphSchema_K2*> Schema, const FPropertyBindingBindableStructDescriptor& InContextStruct)
{
	FEdGraphPinType PinType;

	UStruct* Struct = const_cast<UStruct*>(InContextStruct.Struct.Get());

	if (UClass* Class = Cast<UClass>(Struct))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
		PinType.PinSubCategory = NAME_None;
		PinType.PinSubCategoryObject = Class;
	}
	else if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Struct))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		PinType.PinSubCategory = NAME_None;
		PinType.PinSubCategoryObject = ScriptStruct;
	}

	const FSlateBrush* Icon = FBlueprintEditorUtils::GetIconFromPin(PinType, true);
	const FLinearColor IconColor = Schema->GetPinTypeColor(PinType);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpacer)
			.Size(FVector2D(18.0f, 0.0f))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(1.0f, 0.0f)
		[
			SNew(SImage)
			.Image(Icon)
			.ColorAndOpacity(IconColor)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(FText::FromName(InContextStruct.Name))
		];
}

#undef LOCTEXT_NAMESPACE
