// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBindingUtilsEditorModule.h"

#define LOCTEXT_NAMESPACE "PropertyBindingUtilsEditor"

class FPropertyBindingUtilsEditorModule : public IPropertyBindingUtilsEditorModule
{
};

IMPLEMENT_MODULE(FPropertyBindingUtilsEditorModule, PropertyBindingUtilsEditor)

#undef LOCTEXT_NAMESPACE
