// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SPropertyBindingView.h"
#include "PropertyBindingBinding.h"
#include "PropertyBindingBindingCollection.h"
#include "PropertyBindingDataView.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "SPropertyBindingViewer"

namespace UE::PropertyBinding
{
namespace Private
{
	static const FLazyName ColumnId_SourceStruct = "SourceStruct";
	static const FLazyName ColumnId_SourcePath = "SourcePath";
	static const FLazyName ColumnId_TargetStruct = "TargetStruct";
	static const FLazyName ColumnId_TargetPath = "TargetPath";

	// @todo: We don't check Property Compatibility here as the code is inside FCachedBindingData. We should reuse everything from there(get name, validity).
	// @todo: The compiler may do extra stuff that we cannot implement here. The editor data should have a TArray of invalid binding discovered during compilation.
	class SBindingViewRow : public SMultiColumnTableRow<TSharedPtr<SBindingView::FItem>>
	{
	public:
		SLATE_BEGIN_ARGS(SBindingViewRow) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& Args, TSharedPtr<SBindingView::FItem> InItem, TWeakInterfacePtr<const IPropertyBindingBindingCollectionOwner> InCollectionOwner, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<const SBindingView>& InOwnerBindingView)
		{
			Item = InItem;
			CollectionOwner = InCollectionOwner;
			WeakOwnerBindingView = InOwnerBindingView;
			SMultiColumnTableRow<TSharedPtr<SBindingView::FItem>>::Construct(
				FSuperRowType::FArguments()
				.Padding(1.0f),
				InOwnerTableView
			);
		}

		const FSlateBrush* GetMalformedBindingImage() const
		{
			return FCoreStyle::Get().GetBrush("Icons.ErrorWithColor");
		}

		FText GetMalformedBindingSectionTooltip(bool bIsStruct) const
		{
			return bIsStruct ? LOCTEXT("MissingStructTooltip", "Struct is missing!") : LOCTEXT("MalformedBindingPathTooltip", "Could not find a valid property from the binding path.");
		}

		EVisibility GetMalformedBindingImageVisibility(bool bIsStruct, bool bIsSource) const
		{
			return IsBindingSectionValid(bIsStruct, bIsSource) ? EVisibility::Hidden : EVisibility::Visible;
		}

		FText GetBindingSectionName(bool bIsStruct, bool bIsSource) const
		{
			return HandleGetBindingSectionName(bIsStruct, bIsSource).Key;
		}

		bool IsBindingSectionValid(bool bIsStruct, bool bIsSource) const
		{
			return HandleGetBindingSectionName(bIsStruct, bIsSource).Value;
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			const bool bIsStruct = ColumnName == ColumnId_SourceStruct || ColumnName == ColumnId_TargetStruct;
			const bool bIsSource = ColumnName == ColumnId_SourceStruct || ColumnName == ColumnId_SourcePath;
			const FPropertyBindingPath& BindingPath = bIsSource ? Item->SourcePath : Item->TargetPath;
			const FGuid StructID = bIsSource ? Item->SourcePath.GetStructID() : Item->TargetPath.GetStructID();

			TSharedRef<SWidget> InnerWidget = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
						.HeightOverride(16.0f)
						[
							SNew(SImage)
								.Image(this, &SBindingViewRow::GetMalformedBindingImage)
								.ColorAndOpacity(FLinearColor::White)
								.Visibility(this, &SBindingViewRow::GetMalformedBindingImageVisibility, bIsStruct, bIsSource)
								.ToolTipText(this, &SBindingViewRow::GetMalformedBindingSectionTooltip, bIsStruct)
						]
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(this, &SBindingViewRow::GetBindingSectionName, bIsStruct, bIsSource)
				];

			if (TSharedPtr<const SBindingView> OwnerBindingView = WeakOwnerBindingView.Pin())
			{
				if (OwnerBindingView->OnBindingClicked.IsBound())
				{
					return SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ContentPadding(FMargin(0, 0))
						.OnClicked_Lambda([&BindingPath, bIsStruct, WeakBindingView = WeakOwnerBindingView]()
							{
								if (TSharedPtr<const SBindingView> BindingView = WeakBindingView.Pin())
								{
									BindingView->OnBindingClicked.Execute(bIsStruct, BindingPath);
								}

								return FReply::Handled();
							})
						[
							InnerWidget
						];
				}

				return InnerWidget;
			}

			return SNullWidget::NullWidget;
		}

	private:
		TPair<FText, bool> HandleGetBindingSectionName(bool bIsStruct, bool bIsSource) const
		{
			if (const IPropertyBindingBindingCollectionOwner* CollectionOwnerPtr = CollectionOwner.Get())
			{
				FPropertyBindingPath& BindingPath = bIsSource ? Item->SourcePath : Item->TargetPath;

				if (bIsStruct)
				{
					TInstancedStruct<FPropertyBindingBindableStructDescriptor> StructDesc;
					if (CollectionOwnerPtr->GetBindableStructByID(BindingPath.GetStructID(), StructDesc))
					{
						check(StructDesc.IsValid());

						return { FText::FromString(StructDesc.Get().ToString()), true };
					}

					return { LOCTEXT("MissingStruct", "???"), false };
				}

				bool bIsBindingPathValid = false;
				FPropertyBindingDataView InstanceView;
				if (CollectionOwnerPtr->GetBindingDataViewByID(BindingPath.GetStructID(), InstanceView))
				{
					TArray<FPropertyBindingPathIndirection> Indirections;
					constexpr FString* Error = nullptr;
					constexpr bool bHandleRedirects = true;

					if (BindingPath.ResolveIndirectionsWithValue(InstanceView, Indirections, Error, bHandleRedirects))
					{
						bIsBindingPathValid = true;
					}
				}

				return { FText::FromString(BindingPath.ToString()), bIsBindingPathValid };
			}

			return { FText::GetEmpty(), false };
		}

		TSharedPtr<SBindingView::FItem> Item;
		TWeakInterfacePtr<const IPropertyBindingBindingCollectionOwner> CollectionOwner;

		TWeakPtr<const SBindingView> WeakOwnerBindingView;
	};
}

void SBindingView::Construct(const FArguments& InArgs)
{
	CollectionOwner = InArgs._CollectionOwner;
	OnBindingClicked = InArgs._OnBindingClicked;

	ListView = SNew(SListView<TSharedPtr<FItem>>)
		.SelectionMode(ESelectionMode::None)
		.ListItemsSource(&Values)
		.OnGenerateRow(this, &SBindingView::HandleGenerateRow)
		.HeaderRow
		(
			SNew(SHeaderRow)
			+ SHeaderRow::Column(Private::ColumnId_SourceStruct)
			.DefaultLabel(LOCTEXT("SourceStructColumnLabel", "Source"))
			
			+ SHeaderRow::Column(Private::ColumnId_SourcePath)
			.DefaultLabel(LOCTEXT("SourcePathColumnLabel", "Path"))

			+ SHeaderRow::Column(Private::ColumnId_TargetStruct)
			.DefaultLabel(LOCTEXT("TargetStructColumnLabel", "Target"))

			+ SHeaderRow::Column(Private::ColumnId_TargetPath)
			.DefaultLabel(LOCTEXT("TargetPathColumnLabel", "Path"))
		);

	ChildSlot
	[
		ListView.ToSharedRef()
	];
}

void SBindingView::RequestRefresh()
{
	ListView->RequestListRefresh();
}

void SBindingView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (const IPropertyBindingBindingCollectionOwner* CollectionOwnerPtr = CollectionOwner.Get())
	{
		if (const FPropertyBindingBindingCollection* BindingCollection = CollectionOwnerPtr->GetEditorPropertyBindings())
		{
			int32 Index = 0;
			bool bRequestRefresh = false;
			BindingCollection->ForEachBinding(
				[&Index, &bRequestRefresh, Self = this](const FPropertyBindingBinding& Binding)
				{
					if (Self->Values.Num() <= Index)
					{
						Self->Values.Add(MakeShared<FItem>(Binding.GetSourcePath(), Binding.GetTargetPath(), Binding.GetPropertyFunctionNode().GetScriptStruct()));
						bRequestRefresh = true;
					}
					else
					{
						const TSharedPtr<FItem>& Item = Self->Values[Index];
						const bool bChanged = Item->FunctionNodeStruct.Get() != Binding.GetPropertyFunctionNode().GetScriptStruct()
							|| Item->SourcePath != Binding.GetSourcePath()
							|| Item->TargetPath != Binding.GetTargetPath();
						if (bChanged)
						{
							Self->Values.SetNum(Index);
							Self->Values.Add(MakeShared<FItem>(Binding.GetSourcePath(), Binding.GetTargetPath(), Binding.GetPropertyFunctionNode().GetScriptStruct()));
							bRequestRefresh = true;
						}
					}
					++Index;
				});

			if (Values.Num() != Index)
			{
				Values.SetNum(Index);
				bRequestRefresh = true;
			}

			if (bRequestRefresh)
			{
				RequestRefresh();
			}

			return;
		}
	}

	if (!Values.IsEmpty())
	{
		Values.Empty();
		RequestRefresh();
	}
}

TSharedRef<ITableRow> SBindingView::HandleGenerateRow(TSharedPtr<FItem> Value, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(Private::SBindingViewRow, Value, CollectionOwner, OwnerTable, SharedThis(this));
}

} // namespace UE::PropertyBinding

#undef LOCTEXT_NAMESPACE
