// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class PropertyBindingUtilsEditor : ModuleRules
	{
		public PropertyBindingUtilsEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"BlueprintGraph",
					"Core",
					"CoreUObject",
					"EditorFramework",
					"Engine",
					"InputCore",
					"PropertyBindingUtils",
					"Slate",
					"SlateCore",
					"UnrealEd",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"StructUtilsEditor",
				}
			);
		}
	}
}