// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphPin.h"
#include "IDetailPropertyExtensionHandler.h"
#include "PropertyBindingBindingCollection.h"
#include "PropertyBindingBindingCollectionOwner.h"
#include "PropertyBindingPath.h"
#include "UObject/WeakInterfacePtr.h"

class FMenuBuilder;
class IDetailLayoutBuilder;
class IPropertyAccessEditor;
class IPropertyBindingBindingCollectionOwner;
class IPropertyHandle;
class UEdGraphSchema_K2;
struct FBindingChainElement;
struct FBindingContextStruct;
struct FPropertyBindingBindableStructDescriptor;
struct FPropertyBindingDataView;
struct FPropertyBindingPath;
struct FPropertyBindingBinding;
struct FPropertyBindingWidgetArgs;

namespace UE::PropertyBinding
{
struct FPropertyInfoOverride
{
	/** Display Name Text of the Ref Type */
	FText TypeNameText;

	/** Ref Type expressed as a Pin Type */
	FEdGraphPinType PinType;
};

namespace Meta::Private
{
// valid keywords for the UCLASS, UPROPERTY and USTRUCT macro
enum
{
	// [ClassMetadata] [PropertyMetadata] [StructMetadata] The property cannot be bound to (target of a binding).
	NoBinding,
	// [ClassMetadata] [PropertyMetadata] [StructMetadata] The property cannot be promoted to parameters.
	NoPromoteToParameter,
};
} // namespace Meta::Private

const FLazyName MetaDataStructIDName("StructIDForPropertyBinding");
const FLazyName MetaDataNoBindingName("NoBinding");
const FLazyName MetaDataNoPromoteToParameter("NoPromoteToParameter");

/**
 * Useful if you'd like to cut off some properties from the final path. For example, if your bindable struct descriptors
 * point to structs, and you have these structs wrapped into object nodes on your graph, then you'd annotate the
 * struct property inside the UObject node with both StructIDForPropertyBinding and BindingRoot.
 */
const FLazyName MetaDataBindingRoot("BindingRoot");

UE_DEPRECATED(5.8, "Use the overload with Schema instead.")
PROPERTYBINDINGUTILSEDITOR_API extern FText GetPropertyTypeText(const FProperty* Property);
PROPERTYBINDINGUTILSEDITOR_API extern FText GetPropertyTypeText(TNotNull<const UEdGraphSchema_K2*> Schema, const FProperty* Property);

/**
 * Returns property path for a specific property.
 * Walks towards root up until a property with metadata "MetaDataName" is found.
 * The property's metadata "MetaDataName" is expected to specify the containing struct ID.
 * @param InPropertyHandle Handle to the property to find path for.
 * @param OutPath Resulting property path.
 * @param InFallbackStructID If no bindable struct are found this ID will be used to build the property path.
 * @return property handle of the property found with the required metadata.
 */
PROPERTYBINDINGUTILSEDITOR_API extern TSharedPtr<const IPropertyHandle> MakeStructPropertyPathFromPropertyHandle(
	const TSharedPtr<const IPropertyHandle>& InPropertyHandle
	, FPropertyBindingPath& OutPath
	, FGuid InFallbackStructID = {}
	);

/**
 * Returns true if provided Property is bindable.
 */
PROPERTYBINDINGUTILSEDITOR_API bool IsPropertyBindable(const FProperty& Property);

struct FCachedBindingData
{
	PROPERTYBINDINGUTILSEDITOR_API FCachedBindingData(IPropertyBindingBindingCollectionOwner* InPropertyBindingsOwner
		, const FPropertyBindingPath& InTargetPath
		, const TSharedPtr<const IPropertyHandle>& InPropertyHandle
		, const TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> InAccessibleStructs
		);

	virtual ~FCachedBindingData() = default;

	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API bool HasBinding(FPropertyBindingBindingCollection::ESearchMode SearchMode) const;
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API const FPropertyBindingBinding* FindBinding(FPropertyBindingBindingCollection::ESearchMode SearchMode) const;
	PROPERTYBINDINGUTILSEDITOR_API void AddBinding(TConstArrayView<FBindingChainElement> InBindingChain);
	PROPERTYBINDINGUTILSEDITOR_API void RemoveBinding(FPropertyBindingBindingCollection::ESearchMode RemoveMode);

	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API UStruct* ResolveIndirection(TConstArrayView<FBindingChainElement> InBindingChain);

	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API bool CanAcceptPropertyOrChildren(const FProperty* SourceProperty, TConstArrayView<FBindingChainElement> InBindingChain);

	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API bool CanBindToProperty(const FProperty* SourceProperty, TConstArrayView<FBindingChainElement> InBindingChain);
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API bool CanBindToContextStruct(const UStruct* InStruct, int32 InStructIndex);

	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API bool CanCreateParameter(const FPropertyBindingBindableStructDescriptor& InStructDesc, TArray<TSharedPtr<const UE::PropertyBinding::FPropertyInfoOverride>>& OutPropertyInfoOverrides) const;
	PROPERTYBINDINGUTILSEDITOR_API void PromoteToParameter(FName InPropertyName, TConstStructView<FPropertyBindingBindableStructDescriptor> InStructDesc, TSharedPtr<const FPropertyInfoOverride> InPropertyInfoOverride);

	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API FText GetText();
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API FText GetTooltipText();
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API FLinearColor GetColor();
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API const FSlateBrush* GetImage();

	[[nodiscard]] TConstStructView<FPropertyBindingBindableStructDescriptor> GetBindableStructDescriptor(int32 InStructIndex) const;
	[[nodiscard]] TStructView<FPropertyBindingBindableStructDescriptor> GetMutableBindableStructDescriptor(int32 InStructIndex);
	[[nodiscard]] TWeakObjectPtr<> GetWeakOwner() const;
	[[nodiscard]] UObject* GetOwner() const;
	[[nodiscard]] const IPropertyHandle* GetPropertyHandle() const;
	[[nodiscard]] const FPropertyBindingPath& GetSourcePath() const;
	[[nodiscard]] const FPropertyBindingPath& GetTargetPath() const;
	[[nodiscard]] TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> GetAccessibleStructs() const;
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual TNotNull<const UEdGraphSchema_K2*> GetEdGraphSchema() const;

	[[nodiscard]] FText GetFormatableText() const;
	[[nodiscard]] FText GetFormatableTooltipText() const;

	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool IsPropertyReference(const FProperty& InProperty);
	PROPERTYBINDINGUTILSEDITOR_API virtual void UpdateSourcePropertyPath(TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, const FPropertyBindingPath& InSourcePath, FString& OutString);
	PROPERTYBINDINGUTILSEDITOR_API virtual void UpdatePropertyReferenceTooltip(const FProperty& InProperty, FTextBuilder& InOutTextBuilder) const;
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool CanBindToContextStructInternal(const UStruct* InStruct, int32 InStructIndex);
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool CanAcceptPropertyOrChildrenInternal(const FProperty& InProperty, TConstArrayView<FBindingChainElement> InBindingChain);
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool AddBindingInternal(TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, FPropertyBindingPath& InOutSourcePath, const FPropertyBindingPath& InTargetPath);
	
	PROPERTYBINDINGUTILSEDITOR_API virtual void AddPropertyInfoOverride(const FProperty& InProperty, TArray<TSharedPtr<const UE::PropertyBinding::FPropertyInfoOverride>>& OutPropertyInfoOverrides) const;
	PROPERTYBINDINGUTILSEDITOR_API virtual void GetSourceDataViewForNewBinding(TNotNull<IPropertyBindingBindingCollectionOwner*> InBindingsOwner,TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, FPropertyBindingDataView& OutSourceDataView);

	struct FPinInfo
	{
		FEdGraphPinType Type;
		const FSlateBrush* IconBrush = nullptr;
		FLinearColor IconColor = FLinearColor::White;
	};


	UE_DEPRECATED(5.7, "Use GetPinInfoForProperty instead")
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool GetPinTypeAndIconForProperty(const FProperty& InProperty, FPropertyBindingDataView InTargetDataView, FEdGraphPinType& OutPinType, FName& OutIconName) const final;
	UE_DEPRECATED(5.8, "Use GetPinInfoForProperty instead")
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool GetPinTypeAndIconForProperty(const FProperty& InProperty, FPropertyBindingDataView InTargetDataView, FEdGraphPinType& OutPinType, const FSlateBrush*& OutIconBrush) const final;
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual FPinInfo GetPinInfoForProperty(TNotNull<const FProperty*> InProperty, FPropertyBindingDataView InTargetDataView) const;
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool GetPropertyFunctionText(FConstStructView InPropertyFunctionStructView, FText& OutText) const;
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool GetPropertyFunctionTooltipText(FConstStructView InPropertyFunctionStructView, FText& OutText) const;
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool GetPropertyFunctionIconColor(FConstStructView InPropertyFunctionStructView, FLinearColor& OutColor) const;
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool GetPropertyFunctionImage(FConstStructView InPropertyFunctionStructView, const FSlateBrush*& OutImage) const;
	
	/** @return whether the compatibility has been evaluated. */
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool DeterminePropertiesCompatibilityInternal(
		const FProperty* InSourceProperty,
		const FProperty* InTargetProperty,
		const void* InSourcePropertyValue,
		const void* InTargetPropertyValue,
		bool& bOutAreCompatible) const;
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual bool DeterminePropertyAndContextStructCompatibleInternal(const UStruct* SourceStruct, const FProperty* TargetProperty) const;

	PROPERTYBINDINGUTILSEDITOR_API void UpdateData();

private:
	PROPERTYBINDINGUTILSEDITOR_API void ConditionallyUpdateData();
	PROPERTYBINDINGUTILSEDITOR_API bool ArePropertyAndContextStructCompatible(const UStruct* SourceStruct, const FProperty* TargetProperty) const;
	PROPERTYBINDINGUTILSEDITOR_API bool ArePropertiesCompatible(const FProperty* SourceProperty, const FProperty* TargetProperty, const void* SourcePropertyValue, const void* TargetPropertyValue) const;

	TWeakInterfacePtr<IPropertyBindingBindingCollectionOwner> WeakBindingsOwner = nullptr;
	FPropertyBindingPath CachedSourcePath;
	FPropertyBindingPath TargetPath;
	TSharedPtr<const IPropertyHandle> PropertyHandle;
	TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> AccessibleStructs;

	/* Default name of the source struct. */
	FText SourceStructName;

	/* Binding's display name text. Expects it's source struct name to be injected before use. */
	FText FormatableText;

	/* Binding's tooltip text. Expects it's source struct name to be injected before use. */
	FText FormatableTooltipText;
	
	FLinearColor Color = FLinearColor::White;
	const FSlateBrush* Image = nullptr;

	/** The binding is invalid. The image/color/tooltip/... represents an error state. */
	bool bIsCachedDataValid = false;
	
	bool bIsDataCached = false;
};

inline TConstStructView<FPropertyBindingBindableStructDescriptor> FCachedBindingData::GetBindableStructDescriptor(const int32 InStructIndex) const
{
	check(InStructIndex >= 0 && InStructIndex < AccessibleStructs.Num());
	return AccessibleStructs[InStructIndex];
}

inline TStructView<FPropertyBindingBindableStructDescriptor> FCachedBindingData::GetMutableBindableStructDescriptor(const int32 InStructIndex)
{
	check(InStructIndex >= 0 && InStructIndex < AccessibleStructs.Num());
	return AccessibleStructs[InStructIndex];
}

inline TWeakObjectPtr<> FCachedBindingData::GetWeakOwner() const
{
	return WeakBindingsOwner.GetWeakObjectPtr();
}

inline UObject* FCachedBindingData::GetOwner() const
{
	return WeakBindingsOwner.GetObject();
}

inline const IPropertyHandle* FCachedBindingData::GetPropertyHandle() const
{
	return PropertyHandle.Get();
}

inline const FPropertyBindingPath& FCachedBindingData::GetSourcePath() const
{
	return CachedSourcePath;
}

inline const FPropertyBindingPath& FCachedBindingData::GetTargetPath() const
{
	return TargetPath;
}

inline TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> FCachedBindingData::GetAccessibleStructs() const
{
	return AccessibleStructs;
}

inline FText FCachedBindingData::GetFormatableText() const
{
	return FormatableText;
}

inline FText FCachedBindingData::GetFormatableTooltipText() const
{
	return FormatableTooltipText;
}

} // UE::PropertyBinding

class FPropertyBindingExtension : public IDetailPropertyExtensionHandler
{
public:

	//~ Begin IDetailPropertyExtensionHandler
	PROPERTYBINDINGUTILSEDITOR_API virtual bool IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const override;
	PROPERTYBINDINGUTILSEDITOR_API virtual void ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> InPropertyHandle) override;
	//~ End IDetailPropertyExtensionHandler

protected:
	PROPERTYBINDINGUTILSEDITOR_API virtual TSharedPtr<UE::PropertyBinding::FCachedBindingData> CreateCachedBindingData(IPropertyBindingBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath, const TSharedPtr<IPropertyHandle>& InPropertyHandle, TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> InAccessibleStructs) const;
	PROPERTYBINDINGUTILSEDITOR_API virtual bool CanBindToProperty(const FPropertyBindingPath& InTargetPath, const IPropertyHandle& InPropertyHandle) const;
	PROPERTYBINDINGUTILSEDITOR_API virtual uint64 GetDisallowedPropertyFlags() const;

	/**
	 * The optional override criteria of whether the property can be promoted to parameter 
	 * @param InProperty the property to be tested upon
	 * @param bOutOverride Whether the property should be promoted to parameter, if the return value is true
	 * @return Whether the promotion criteria is overridden for this property
	 */
	PROPERTYBINDINGUTILSEDITOR_API virtual bool GetPromotionToParameterOverrideInternal(const FProperty& InProperty, bool& bOutOverride) const;
	PROPERTYBINDINGUTILSEDITOR_API virtual void UpdateContextStruct(TConstStructView<FPropertyBindingBindableStructDescriptor> InStructDesc, FBindingContextStruct& InOutContextStruct, TMap<FString, FText>& InOutSectionNames) const;
	PROPERTYBINDINGUTILSEDITOR_API virtual void CustomizeDetailWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, IPropertyBindingBindingCollectionOwner* InBindingsOwner, TSharedPtr<IPropertyHandle> InPropertyHandle, const FPropertyBindingPath& InTargetPath, TSharedPtr<UE::PropertyBinding::FCachedBindingData> InCachedBindingData) const;

	PROPERTYBINDINGUTILSEDITOR_API virtual void CustomizePropertyBindingWidgetArgs(FPropertyBindingWidgetArgs& Args, IPropertyBindingBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath, TSharedPtr<UE::PropertyBinding::FCachedBindingData> InCachedBindingData);

	UE_DEPRECATED(5.7, "CustomizeWidgetArgs is deprecated. Use CustomizePropertyBindingWidgetArgs instead.")
	virtual void CustomizeWidgetArgs(FPropertyBindingWidgetArgs& Args, IPropertyBindingBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath, TSharedPtr<UE::PropertyBinding::FCachedBindingData> InCachedBindingData) final
	{
		CustomizePropertyBindingWidgetArgs(Args, InBindingsOwner, InTargetPath, InCachedBindingData);
	}

	PROPERTYBINDINGUTILSEDITOR_API virtual bool CanBindToArrayElements() const;
	[[nodiscard]] PROPERTYBINDINGUTILSEDITOR_API virtual TNotNull<const UEdGraphSchema_K2*> GetEdGraphSchema() const;

	PROPERTYBINDINGUTILSEDITOR_API void WrapValueWidget(FDetailWidgetRow& InWidgetRow, const TSharedPtr<UE::PropertyBinding::FCachedBindingData>& InCachedBindingData) const;
	PROPERTYBINDINGUTILSEDITOR_API void CustomizeResetToDefault(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const TSharedPtr<IPropertyHandle>& InPropertyHandle, const TSharedPtr<UE::PropertyBinding::FCachedBindingData>& InCachedBindingData) const;

	UE_DEPRECATED(5.8, "Use the overload with Schema instead.")
	PROPERTYBINDINGUTILSEDITOR_API static TSharedRef<SWidget> MakeContextStructWidget(const FPropertyBindingBindableStructDescriptor& InContextStruct);
	PROPERTYBINDINGUTILSEDITOR_API static TSharedRef<SWidget> MakeContextStructWidget(TNotNull<const UEdGraphSchema_K2*> Schema, const FPropertyBindingBindableStructDescriptor& InContextStruct);

private:
	bool CanPromoteToParameter(const TSharedPtr<IPropertyHandle>& InPropertyHandle) const;
};
