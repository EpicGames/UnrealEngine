// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingBinding.h"
#include "PropertyBindingBindingCollectionOwner.h"
#include "UObject/WeakInterfacePtr.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

struct FPropertyBindingBindingCollection;

namespace UE::PropertyBinding
{
	namespace Private
	{
		class SBindingViewRow;
	}

	class SBindingView : public SCompoundWidget
	{
	private:
		friend Private::SBindingViewRow;
		struct FItem
		{
			FPropertyBindingPath SourcePath;
			FPropertyBindingPath TargetPath;
			TWeakObjectPtr<const UScriptStruct> FunctionNodeStruct;
		};

	public:
		DECLARE_DELEGATE_RetVal(const FPropertyBindingBindingCollection*, FOnGetBindingCollection);
		DECLARE_DELEGATE_TwoParams(FOnBindingClicked, bool bInIsStruct, const FPropertyBindingPath& InBindingPath);

		SLATE_BEGIN_ARGS(SBindingView){}
			UE_DEPRECATED(5.8, "Use CollectionOwner->GetEditorPropertyBindings() instead")
			SLATE_EVENT(FOnGetBindingCollection, GetBindingCollection)

			SLATE_EVENT(FOnBindingClicked, OnBindingClicked)
			SLATE_ARGUMENT(TScriptInterface<const IPropertyBindingBindingCollectionOwner>, CollectionOwner)
		SLATE_END_ARGS()

		PROPERTYBINDINGUTILSEDITOR_API void Construct(const FArguments& InArgs);
		PROPERTYBINDINGUTILSEDITOR_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	private:
		void RequestRefresh();

		TSharedRef<ITableRow> HandleGenerateRow(TSharedPtr<FItem> Value, const TSharedRef<STableViewBase>& OwnerTable);

	private:
		FOnBindingClicked OnBindingClicked;

		TWeakInterfacePtr<const IPropertyBindingBindingCollectionOwner> CollectionOwner;
		TSharedPtr<SListView<TSharedPtr<FItem>>> ListView;
		TArray<TSharedPtr<FItem>> Values;
};

} // namespace UE::PropertyBinding