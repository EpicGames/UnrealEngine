// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBindingUtilsTest.h"
#include "PropertyBindingPath.h"
#include "AITestsCommon.h"
#include "PropertyBindingDataView.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyBindingUtilsTest)

#define LOCTEXT_NAMESPACE "AITestSuite_PropertyBindingUtilsTest"

UE_DISABLE_OPTIMIZATION_SHIP


struct FPropertyBindingUtilsTest_PropertyPathOffset : FAITestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("StructB.B"));

		AITEST_TRUE("Parsing path should succeeed", bParseResult);
		AITEST_EQUAL("Should have 2 path segments", Path.NumSegments(), 2);

		FString ResolveErrors;
		TArray<FPropertyBindingPathIndirection> Indirections;
		const bool bResolveResult = Path.ResolveIndirections(FPropertyBindingUtilsTest_PropertyStruct::StaticStruct(), Indirections, &ResolveErrors);

		AITEST_TRUE("Resolve path should succeeed", bResolveResult);
		AITEST_EQUAL("Should have no resolve errors", ResolveErrors.Len(), 0);
		
		AITEST_EQUAL("Should have 2 indirections", Indirections.Num(), 2);
		AITEST_EQUAL("Indirection 0 should be Offset type", Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);
		AITEST_EQUAL("Indirection 1 should be Offset type", Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FPropertyBindingUtilsTest_PropertyPathOffset, "System.PropertyBindingUtils.Offset");

struct FPropertyBindingUtilsTest_PropertyPathParseFail : FAITestBase
{
	virtual bool InstantTest() override
	{
		{
			FPropertyBindingPath Path;
			const bool bParseResult = Path.FromString(TEXT("")); // empty is valid.
			AITEST_TRUE("Parsing path should succeed", bParseResult);
		}

		{
			FPropertyBindingPath Path;
			const bool bParseResult = Path.FromString(TEXT("StructB.[0]B"));
			AITEST_FALSE("Parsing path should fail", bParseResult);
		}

		{
			FPropertyBindingPath Path;
			const bool bParseResult = Path.FromString(TEXT("StructB..NoThere"));
			AITEST_FALSE("Parsing path should fail", bParseResult);
		}

		{
			FPropertyBindingPath Path;
			const bool bParseResult = Path.FromString(TEXT("."));
			AITEST_FALSE("Parsing path should fail", bParseResult);
		}

		{
			FPropertyBindingPath Path;
			const bool bParseResult = Path.FromString(TEXT("StructB..B"));
			AITEST_FALSE("Parsing path should fail", bParseResult);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FPropertyBindingUtilsTest_PropertyPathParseFail, "System.PropertyBindingUtils.ParseFail");

struct FPropertyBindingUtilsTest_PropertyPathOffsetFail : FAITestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("StructB.Q"));

		AITEST_TRUE("Parsing path should succeeed", bParseResult);
		AITEST_EQUAL("Should have 2 path segments", Path.NumSegments(), 2);

		FString ResolveErrors;
		TArray<FPropertyBindingPathIndirection> Indirections;
		const bool bResolveResult = Path.ResolveIndirections(FPropertyBindingUtilsTest_PropertyStruct::StaticStruct(), Indirections, &ResolveErrors);

		AITEST_FALSE("Resolve path should not succeeed", bResolveResult);
		AITEST_NOT_EQUAL("Should have errors", ResolveErrors.Len(), 0);
		
		AITEST_EQUAL("Should have 0 indirections", Indirections.Num(), 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FPropertyBindingUtilsTest_PropertyPathOffsetFail, "System.PropertyBindingUtils.OffsetFail");

struct FPropertyBindingUtilsTest_PropertyPathObject : FAITestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("InstancedObject.A"));

		AITEST_TRUE("Parsing path should succeeed", bParseResult);
		AITEST_EQUAL("Should have 2 path segments", Path.NumSegments(), 2);

		UPropertyBindingUtilsTest_PropertyObject* Object = NewObject<UPropertyBindingUtilsTest_PropertyObject>();
		Object->InstancedObject = NewObject<UPropertyBindingUtilsTest_PropertyObjectInstanced>();
		
		const bool bUpdateResult = Path.UpdateSegmentsFromValue(FPropertyBindingDataView(Object));

		AITEST_TRUE("Update instance types should succeeed", bUpdateResult);
		AITEST_TRUE("Path segment 0 instance type should be UPropertyBindingUtilsTest_PropertyObjectInstanced", Path.GetSegment(0).GetInstanceStruct() == UPropertyBindingUtilsTest_PropertyObjectInstanced::StaticClass());
		AITEST_TRUE("Path segment 1 instance type should be nullptr", Path.GetSegment(1).GetInstanceStruct() == nullptr);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FPropertyBindingUtilsTest_PropertyPathObject, "System.PropertyBindingUtils.Object");

struct FPropertyBindingUtilsTest_PropertyPathWrongObject : FAITestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("InstancedObject.B"));

		AITEST_TRUE("Parsing path should succeeed", bParseResult);
		AITEST_EQUAL("Should have 2 path segments", Path.NumSegments(), 2);

		UPropertyBindingUtilsTest_PropertyObject* Object = NewObject<UPropertyBindingUtilsTest_PropertyObject>();

		Object->InstancedObject = NewObject<UPropertyBindingUtilsTest_PropertyObjectInstancedWithB>();
		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path.ResolveIndirectionsWithValue(FPropertyBindingDataView(Object), Indirections, &ResolveErrors);

			AITEST_TRUE("Resolve path should succeeed", bResolveResult);
			AITEST_EQUAL("Should have 2 indirections", Indirections.Num(), 2);
			AITEST_TRUE("Object ", Indirections[0].GetAccessType() == EPropertyBindingPropertyAccessType::ObjectInstance);
			AITEST_TRUE("Object ", Indirections[0].GetContainerStruct() == Object->GetClass());
			AITEST_TRUE("Object ", Indirections[0].GetInstanceStruct() == UPropertyBindingUtilsTest_PropertyObjectInstancedWithB::StaticClass());
			AITEST_EQUAL("Should not have error", ResolveErrors.Len(), 0);
		}

		Object->InstancedObject = NewObject<UPropertyBindingUtilsTest_PropertyObjectInstanced>();
		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path.ResolveIndirectionsWithValue(FPropertyBindingDataView(Object), Indirections, &ResolveErrors);

			AITEST_FALSE("Resolve path should fail", bResolveResult);
			AITEST_EQUAL("Should have 0 indirections", Indirections.Num(), 0);
			AITEST_NOT_EQUAL("Should have error", ResolveErrors.Len(), 0);
		}
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FPropertyBindingUtilsTest_PropertyPathWrongObject, "System.PropertyBindingUtils.WrongObject");

struct FPropertyBindingUtilsTest_PropertyPathArray : FAITestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("ArrayOfInts[1]"));

		AITEST_TRUE("Parsing path should succeeed", bParseResult);
		AITEST_EQUAL("Should have 1 path segments", Path.NumSegments(), 1);

		UPropertyBindingUtilsTest_PropertyObject* Object = NewObject<UPropertyBindingUtilsTest_PropertyObject>();
		Object->ArrayOfInts.Add(42);
		Object->ArrayOfInts.Add(123);

		FString ResolveErrors;
		TArray<FPropertyBindingPathIndirection> Indirections;
		const bool bResolveResult = Path.ResolveIndirectionsWithValue(FPropertyBindingDataView(Object), Indirections, &ResolveErrors);

		AITEST_TRUE("Resolve path should succeeed", bResolveResult);
		AITEST_EQUAL("Should have no resolve errors", ResolveErrors.Len(), 0);
		AITEST_EQUAL("Should have 2 indirections", Indirections.Num(), 2);
		AITEST_EQUAL("Indirection 0 should be IndexArray type", Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::IndexArray);
		AITEST_EQUAL("Indirection 1 should be Offset type", Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);

		const int32 Value = *reinterpret_cast<const int32*>(Indirections[1].GetPropertyAddress());
		AITEST_EQUAL("Value should be 123", Value, 123);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FPropertyBindingUtilsTest_PropertyPathArray, "System.PropertyBindingUtils.Array");

struct FPropertyBindingUtilsTest_PropertyPathArrayInvalidIndex : FAITestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("ArrayOfInts[123]"));

		AITEST_TRUE("Parsing path should succeeed", bParseResult);
		AITEST_EQUAL("Should have 1 path segments", Path.NumSegments(), 1);

		UPropertyBindingUtilsTest_PropertyObject* Object = NewObject<UPropertyBindingUtilsTest_PropertyObject>();
		Object->ArrayOfInts.Add(42);
		Object->ArrayOfInts.Add(123);

		FString ResolveErrors;
		TArray<FPropertyBindingPathIndirection> Indirections;
		const bool bResolveResult = Path.ResolveIndirectionsWithValue(FPropertyBindingDataView(Object), Indirections, &ResolveErrors);

		AITEST_FALSE("Resolve path should fail", bResolveResult);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FPropertyBindingUtilsTest_PropertyPathArrayInvalidIndex, "System.PropertyBindingUtils.ArrayInvalidIndex");

struct FPropertyBindingUtilsTest_PropertyPathArrayOfStructs : FAITestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path1;
		Path1.FromString(TEXT("ArrayOfStruct[0].B"));

		FPropertyBindingPath Path2;
		Path2.FromString(TEXT("ArrayOfStruct[2].StructB.B"));

		UPropertyBindingUtilsTest_PropertyObject* Object = NewObject<UPropertyBindingUtilsTest_PropertyObject>();
		Object->ArrayOfStruct.AddDefaulted_GetRef().B = 3;
		Object->ArrayOfStruct.AddDefaulted();
		Object->ArrayOfStruct.AddDefaulted_GetRef().StructB.B = 42;

		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path1.ResolveIndirectionsWithValue(FPropertyBindingDataView(Object), Indirections, &ResolveErrors);

			AITEST_TRUE("Resolve path1 should succeeed", bResolveResult);
			AITEST_EQUAL("Should have no resolve errors", ResolveErrors.Len(), 0);
			AITEST_EQUAL("Should have 3 indirections", Indirections.Num(), 3);
			AITEST_EQUAL("Indirection 0 should be ArrayIndex type", Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::IndexArray);
			AITEST_EQUAL("Indirection 1 should be Offset type", Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);
			AITEST_EQUAL("Indirection 2 should be Offset type", Indirections[2].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);

			const int32 Value = *reinterpret_cast<const int32*>(Indirections[2].GetPropertyAddress());
			AITEST_EQUAL("Value should be 3", Value, 3);
		}

		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path2.ResolveIndirectionsWithValue(FPropertyBindingDataView(Object), Indirections, &ResolveErrors);

			AITEST_TRUE("Resolve path2 should succeeed", bResolveResult);
			AITEST_EQUAL("Should have no resolve errors", ResolveErrors.Len(), 0);
			AITEST_EQUAL("Should have 4 indirections", Indirections.Num(), 4);
			AITEST_EQUAL("Indirection 0 should be ArrayIndex type", Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::IndexArray);
			AITEST_EQUAL("Indirection 1 should be Offset type", Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);
			AITEST_EQUAL("Indirection 2 should be Offset type", Indirections[2].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);
			AITEST_EQUAL("Indirection 3 should be Offset type", Indirections[3].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);

			const int32 Value = *reinterpret_cast<const int32*>(Indirections[3].GetPropertyAddress());
			AITEST_EQUAL("Value should be 42", Value, 42);
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FPropertyBindingUtilsTest_PropertyPathArrayOfStructs, "System.PropertyBindingUtils.ArrayOfStructs");

struct FPropertyBindingUtilsTest_PropertyPathArrayOfInstancedObjects : FAITestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		Path.FromString(TEXT("ArrayOfInstancedStructs[0].B"));

		FPropertyBindingUtilsTest_PropertyStruct Struct;
		Struct.B = 123;
		
		UPropertyBindingUtilsTest_PropertyObject* Object = NewObject<UPropertyBindingUtilsTest_PropertyObject>();
		Object->ArrayOfInstancedStructs.Emplace(FConstStructView::Make(Struct));

		const bool bUpdateResult = Path.UpdateSegmentsFromValue(FPropertyBindingDataView(Object));
		AITEST_TRUE("Update instance types should succeeed", bUpdateResult);
		AITEST_EQUAL("Should have 2 path segments", Path.NumSegments(), 2);
		AITEST_TRUE("Path segment 0 instance type should be FPropertyBindingUtilsTest_PropertyStruct", Path.GetSegment(0).GetInstanceStruct() == FPropertyBindingUtilsTest_PropertyStruct::StaticStruct());
		AITEST_TRUE("Path segment 1 instance type should be nullptr", Path.GetSegment(1).GetInstanceStruct() == nullptr);

		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path.ResolveIndirections(UPropertyBindingUtilsTest_PropertyObject::StaticClass(), Indirections, &ResolveErrors);

			AITEST_TRUE("Resolve path should succeeed", bResolveResult);
			AITEST_EQUAL("Should have no resolve errors", ResolveErrors.Len(), 0);
			AITEST_EQUAL("Should have 3 indirections", Indirections.Num(), 3);
			AITEST_EQUAL("Indirection 0 should be ArrayIndex type", Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::IndexArray);
			AITEST_EQUAL("Indirection 1 should be StructInstance type", Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::StructInstance);
			AITEST_EQUAL("Indirection 2 should be Offset type", Indirections[2].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);
		}

		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path.ResolveIndirectionsWithValue(FPropertyBindingDataView(Object), Indirections, &ResolveErrors);

			AITEST_TRUE("Resolve path should succeeed", bResolveResult);
			AITEST_EQUAL("Should have no resolve errors", ResolveErrors.Len(), 0);
			AITEST_EQUAL("Should have 3 indirections", Indirections.Num(), 3);
			AITEST_EQUAL("Indirection 0 should be ArrayIndex type", Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::IndexArray);
			AITEST_EQUAL("Indirection 1 should be StructInstance type", Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::StructInstance);
			AITEST_EQUAL("Indirection 2 should be Offset type", Indirections[2].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);

			const int32 Value = *reinterpret_cast<const int32*>(Indirections[2].GetPropertyAddress());
			AITEST_EQUAL("Value should be 123", Value, 123);
		}
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FPropertyBindingUtilsTest_PropertyPathArrayOfInstancedObjects, "System.PropertyBindingUtils.ArrayOfInstancedObjects");


UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE

