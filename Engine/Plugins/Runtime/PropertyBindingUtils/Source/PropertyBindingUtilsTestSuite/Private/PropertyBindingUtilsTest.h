// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "PropertyBindingUtilsTest.generated.h"


USTRUCT()
struct FPropertyBindingUtilsTest_PropertyStructB
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "")
	int32 B = 0;
};

USTRUCT()
struct FPropertyBindingUtilsTest_PropertyStruct
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "")
	int32 A = 0;

	UPROPERTY(EditAnywhere, Category = "")
	int32 B = 0;

	UPROPERTY(EditAnywhere, Category = "")
	FPropertyBindingUtilsTest_PropertyStructB StructB;
};

UCLASS(HideDropdown)
class UPropertyBindingUtilsTest_PropertyObjectInstanced : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "")
	int32 A = 0;

	UPROPERTY(EditAnywhere, Category = "")
	FInstancedStruct InstancedStruct;
};

UCLASS(HideDropdown)
class UPropertyBindingUtilsTest_PropertyObjectInstancedWithB : public UPropertyBindingUtilsTest_PropertyObjectInstanced
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "")
	int32 B = 0;
};

UCLASS(HideDropdown)
class UPropertyBindingUtilsTest_PropertyObject : public UObject
{
	GENERATED_BODY()
public:
	
	UPROPERTY(EditAnywhere, Instanced, Category = "")
	TObjectPtr<UPropertyBindingUtilsTest_PropertyObjectInstanced> InstancedObject;

	UPROPERTY(EditAnywhere, Instanced, Category = "")
	TArray<TObjectPtr<UPropertyBindingUtilsTest_PropertyObjectInstanced>> ArrayOfInstancedObjects;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<int32> ArrayOfInts;

	UPROPERTY(EditAnywhere, Category = "")
	FInstancedStruct InstancedStruct;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<FInstancedStruct> ArrayOfInstancedStructs;

	UPROPERTY(EditAnywhere, Category = "")
	FPropertyBindingUtilsTest_PropertyStruct Struct;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<FPropertyBindingUtilsTest_PropertyStruct> ArrayOfStruct;
};