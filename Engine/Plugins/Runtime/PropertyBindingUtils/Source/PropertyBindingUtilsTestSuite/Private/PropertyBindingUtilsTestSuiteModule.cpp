// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBindingUtilsTestSuiteModule.h"

#define LOCTEXT_NAMESPACE "PropertyBindingUtilsTestSuite"

class FPropertyBindingUtilsTestSuiteModule : public IPropertyBindingUtilsTestSuiteModule
{
};

IMPLEMENT_MODULE(FPropertyBindingUtilsTestSuiteModule, PropertyBindingUtilsTestSuite)

#undef LOCTEXT_NAMESPACE
